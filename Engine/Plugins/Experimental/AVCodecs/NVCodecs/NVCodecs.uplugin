{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "NVCodecs",
	"Description": "Adds codecs from the NVIDIA Media Codec SDK to AVCodecs",
	"Category": "Codecs",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "NVCodecs",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"TargetDenyList": [
				"Server"
			],
			"PlatformAllowList": [ "Win64",	"Linux" ]
		},
		{
			"Name": "NVCodecsRHI",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"TargetDenyList": [
				"Server"
			],
			"PlatformAllowList": [ "Win64",	"Linux" ]
		},
		{
			"Name": "NVDEC",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"TargetDenyList": [
				"Server"
			],
			"PlatformAllowList": [ "Win64",	"Linux" ]
		},
		{
			"Name": "NVENC",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"TargetDenyList": [
				"Server"
			],
			"PlatformAllowList": [ "Win64",	"Linux" ]
		}
	],
	"Plugins": [
		{
			"Name": "AVCodecsCore",
			"Enabled": true
		}
	],
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux"
	]
}