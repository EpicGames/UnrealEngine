// Copyright Epic Games, Inc. All Rights Reserved.

#include "CudaModule.h"

#include "cuda.h"

/*
CUresult NV12_to_BGRA8_Device(CUdeviceptr SrcI420, CUdeviceptr DstABGR8, uint32 Width, uint32 Height, uint32 Pitch)
{
	// HACK do this once instead
	const char* module_file = "Kernels/nv12_to_bgra8.fatbin";
	const char* kernel_name = "nv12_to_bgra8_device";

	CUmodule module;
	CUfunction nv12_to_bgra8_func;
	CUresult Result;

	// HACK: Won't work in packaged builds
	FString const module_file_path = FPaths::Combine(FPaths::EnginePluginsDir(), TEXT("Media/AVCodecs/NVCodec/Source/NVCodec/Kernels/lib/NV12_to_BGRA8.fatbin"));

	Result = FCUDAModule::CUDA().cuModuleLoad(&module, TCHAR_TO_UTF8(*module_file_path));
	if (Result != CUDA_SUCCESS)
	{
		UE_LOG(LogTemp, Fatal, TEXT("Failed to open %s"), *module_file_path);
	}

	Result = FCUDAModule::CUDA().cuModuleGetFunction(&nv12_to_bgra8_func, module, kernel_name);
	if (Result != CUDA_SUCCESS)
	{
		UE_LOG(LogTemp, Fatal, TEXT("Get module function %s failed (%d)"), UTF8_TO_TCHAR(kernel_name), Result);
	}

	void* params[5] = { &SrcI420, &DstABGR8, &Width, &Height, &Pitch };

	Result = FCUDAModule::CUDA().cuLaunchKernel(nv12_to_bgra8_func,
											   	Width / 64 + 1, Height / 64 + 1, 1,
												32, 32, 1,
												0, 0, params, NULL);
	if (Result != CUDA_SUCCESS)
	{
		UE_LOG(LogTemp, Fatal, TEXT("Launching kernel %s failed (%d)"), UTF8_TO_TCHAR(kernel_name), Result);
	}

	Result = FCUDAModule::CUDA().cuStreamSynchronize(0);
	if (Result != CUDA_SUCCESS)
	{
		UE_LOG(LogTemp, Fatal, TEXT("Synchronising kernel %s failed (%d)"), UTF8_TO_TCHAR(kernel_name), Result);
	}

	return Result;
}
*/

CUresult P010_to_ABGR10(CUdeviceptr SrcI420, CUarray DstABGR8, uint32 Width, uint32 Height, uint32 Pitch)
{
	// HACK do this once instead
	const char* module_file = "Kernels/lib/P010_to_ABGR10.fatbin";
	const char* kernel_name = "p010_to_abgr10";

	CUmodule module;
	CUfunction p010_to_abgr10_func;
	CUresult Result;

	// HACK: Won't work in packaged builds
	FString const module_file_path = FPaths::Combine(FPaths::EnginePluginsDir(), TEXT("Media/AVCodecs/NVCodecs/Source/NVCodecs/"), UTF8_TO_TCHAR(module_file));

	Result = FCUDAModule::CUDA().cuModuleLoad(&module, TCHAR_TO_UTF8(*module_file_path));
	if (Result != CUDA_SUCCESS)
	{
		UE_LOG(LogTemp, Fatal, TEXT("Failed to open %s"), *module_file_path);
	}

	Result = FCUDAModule::CUDA().cuModuleGetFunction(&p010_to_abgr10_func, module, kernel_name);
	if (Result != CUDA_SUCCESS)
	{
		UE_LOG(LogTemp, Fatal, TEXT("Get module function %s failed (%d)"), UTF8_TO_TCHAR(kernel_name), Result);
	}

	// Specify texture
	CUDA_RESOURCE_DESC resDesc;
	memset(&resDesc, 0, sizeof(resDesc));
	resDesc.resType = CUresourcetype::CU_RESOURCE_TYPE_ARRAY;
	resDesc.res.array.hArray = DstABGR8;
	
	// Create the surface objects
	CUsurfObject outputSurfObj = 0;
	FCUDAModule::CUDA().cuSurfObjectCreate(&outputSurfObj, &resDesc);

	void* params[5] = { &SrcI420, &outputSurfObj, &Width, &Height, &Pitch };

	Result = FCUDAModule::CUDA().cuLaunchKernel(p010_to_abgr10_func,
											   	Width / 64 + 1, Height / 64 + 1, 1,
												32, 32, 1,
												0, 0, params, NULL);
	if (Result != CUDA_SUCCESS)
	{
		UE_LOG(LogTemp, Fatal, TEXT("Launching kernel %s failed (%d)"), UTF8_TO_TCHAR(kernel_name), Result);
	}

	Result = FCUDAModule::CUDA().cuStreamSynchronize(0);
	if (Result != CUDA_SUCCESS)
	{
		UE_LOG(LogTemp, Fatal, TEXT("Synchronising kernel %s failed (%d)"), UTF8_TO_TCHAR(kernel_name), Result);
	}

	FCUDAModule::CUDA().cuSurfObjectDestroy(outputSurfObj);

	return Result;
}