// Copyright Epic Games, Inc. All Rights Reserved.

#include "cuda.h"

extern "C" __global__ void nv12_to_bgra8_device(uint8_t* SrcNV12, uchar4* DstABGR8, uint32_t const Width, uint32_t const Height, uint32_t const Pitch)
{
	uint32_t x, y, pixel, uvIndex;

	// 1 thread per 2x2 output pixels block
	x = (blockIdx.x * blockDim.x) + threadIdx.x;
	y = (blockIdx.y * blockDim.y) + threadIdx.y;

	// early exit
	if (2 * x >= Width || 2 * y >= Height)
	{
		return;
	}

	// calculate pixel offset
	float r, g, b, cb, cr;

	uvIndex = Pitch * Height + Pitch * y + x * 2;

	cb = SrcNV12[uvIndex] - 128;
	cr = SrcNV12[uvIndex + 1] - 128;

	// calc BGR values
	b = 1.732446f * cb;
	g = -0.337633f * cb - 0.698001f * cr;
	r = 1.370705f * cr;

	// fetch and set each BGRA pixel
	float yp;
	uint32_t xOut, yOut;

#pragma unroll
	for (uint8_t j = 0; j < 2; j++)
	{
		yOut = y * 2 + j;
#pragma unroll
		for (uint8_t i = 0; i < 2; i++)
		{
			xOut = x * 2 + i;

			pixel = Pitch * yOut + xOut;

			yp = SrcNV12[pixel];

			// ABGR
			DstABGR8[xOut + yOut * Width] = make_uchar4(
							min(max(int(yp + b), 0), 255),
					 		min(max(int(yp + g), 0), 255),
							min(max(int(yp + r), 0), 255),
							255);
		}
	}
}

extern "C" __global__ void nv12_to_bgra8(uint8_t* SrcNV12, cudaSurfaceObject_t DstABGR8, uint32_t const Width, uint32_t const Height, uint32_t const Pitch)
{
	uint32_t x, y, pixel, uvIndex;

	// 1 thread per 2x2 output pixels block
	x = (blockIdx.x * blockDim.x) + threadIdx.x;
	y = (blockIdx.y * blockDim.y) + threadIdx.y;

	// early exit
	if (2 * x >= Width || 2 * y >= Height)
	{
		return;
	}

	// calculate pixel offset
	float r, g, b, cb, cr;

	uvIndex = Pitch * Height + Pitch * y + x * 2;

	cb = SrcNV12[uvIndex] - 128;
	cr = SrcNV12[uvIndex + 1] - 128;

	// calc BGR values
	b = 1.732446f * cb;
	g = -0.337633f * cb - 0.698001f * cr;
	r = 1.370705f * cr;

	// fetch and set each BGRA pixel
	float yp;
	uint32_t xOut, yOut;

#pragma unroll
	for (uint8_t j = 0; j < 2; j++)
	{
		yOut = y * 2 + j;
#pragma unroll
		for (uint8_t i = 0; i < 2; i++)
		{
			xOut = x * 2 + i;

			pixel = Pitch * yOut + xOut;

			yp = SrcNV12[pixel];

			// ABGR
			uchar4 data = { min(max(int(yp + b), 0), 255),
					 		min(max(int(yp + g), 0), 255),
							min(max(int(yp + r), 0), 255),
							255};

			surf2Dwrite(data, DstABGR8, xOut * sizeof(uchar4), yOut);
		}
	}
}

extern "C" __global__ void nv12_to_r8(uint8_t* SrcNV12, cudaSurfaceObject_t DstABGR8, uint32_t const Width, uint32_t const Height, uint32_t const Pitch)
{
	uint32_t x, y, pixel, uvIndex;

	// 1 thread per 4 output pixels
	x = (blockIdx.x * blockDim.x) + threadIdx.x;
	y = (blockIdx.y * blockDim.y) + threadIdx.y;

	// early exit
	if (2 * x >= Width || 2 * y >= Height)
	{
		return;
	}

	// calculate pixel offset
	float r, g, b, cb, cr;

	uvIndex = Pitch * Height + Pitch * y + x * 2;

	cb = SrcNV12[uvIndex] - 128;
	cr = SrcNV12[uvIndex + 1] - 128;

	// calc BGR values
	b = 2.017 * cb;
	g = -0.392 * cb - 0.813 * cr;
	r = 1.596 * cr;

	// fetch and set each BGRA pixel
	float yp;
	uint32_t xOut, yOut;

#pragma unroll
	for (uint8_t j = 0; j < 2; j++)
	{
		yOut = y * 2 + j;
#pragma unroll
		for (uint8_t i = 0; i < 2; i++)
		{
			xOut = (x * 2 + i) * 4;

			pixel = Pitch * yOut + xOut;

			yp = 1.164 * (SrcNV12[pixel] - 16);

			// ABGR
			unsigned char data;
#pragma unroll
			for (uint8_t c = 0; c < 4; c++)
			{
				switch(c)
				{
					case 0: // B
						data = min(max(int(yp + b), 0), 255);
						break;
					case 1: // G
						data = min(max(int(yp + g), 0), 255);
						break;
					case 2: // R
						data = min(max(int(yp + r), 0), 255);
						break;
					default: // A
						data = 255u;
				}

				surf2Dwrite(data, DstABGR8, (xOut + c) * sizeof(unsigned char), yOut);
			}
		}
	}
}