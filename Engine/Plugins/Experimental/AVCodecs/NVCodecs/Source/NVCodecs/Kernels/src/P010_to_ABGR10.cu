// Copyright Epic Games, Inc. All Rights Reserved.

#include "cuda.h"

/*
extern "C" __global__ void nv12_to_bgra8_device(uint8_t* SrcNV12, uchar4* DstABGR8, uint32_t const Width, uint32_t const Height, uint32_t const Pitch)
{
	uint32_t x, y, pixel, uvIndex;

	// 1 thread per 2x2 output pixels block
	x = (blockIdx.x * blockDim.x) + threadIdx.x;
	y = (blockIdx.y * blockDim.y) + threadIdx.y;

	// early exit
	if (2 * x >= Width || 2 * y >= Height)
	{
		return;
	}

	// calculate pixel offset
	float r, g, b, cb, cr;

	uvIndex = Pitch * Height + Pitch * y + x * 2;

	cb = SrcNV12[uvIndex] - 128;
	cr = SrcNV12[uvIndex + 1] - 128;

	// calc BGR values
	b = 1.732446f * cb;
	g = -0.337633f * cb - 0.698001f * cr;
	r = 1.370705f * cr;

	// fetch and set each BGRA pixel
	float yp;
	uint32_t xOut, yOut;

#pragma unroll
	for (uint8_t j = 0; j < 2; j++)
	{
		yOut = y * 2 + j;
#pragma unroll
		for (nv12_to_bgra8_funcixel = Pitch * yOut + xOut;

			yp = SrcNV12[pixel];

			// ABGR
			DstABGR8[xOut + yOut * Width] = make_uchar4(
							min(max(int(yp + b), 0), 255),
					 		min(max(int(yp + g), 0), 255),
							min(max(int(yp + r), 0), 255),
							255);
		}
	}
}
*/

// TODO do color correction here
extern "C" __global__ void p010_to_abgr10(uint16_t* SrcP010, cudaSurfaceObject_t DstABGR10, uint32_t const Width, uint32_t const Height, uint32_t const Pitch)
{
	uint32_t x, y, pixel, uvIndex;

	// 1 thread per 2x2 output pixels block
	x = (blockIdx.x * blockDim.x) + threadIdx.x;
	y = (blockIdx.y * blockDim.y) + threadIdx.y;

	// early exit
	if (2 * x >= Width || 2 * y >= Height)
	{
		return;
	}

	// calculate pixel offset
	float r, g, b, cb, cr;

	uvIndex = (Pitch / 2) * Height + Width * y + x * 2;

	// we want 10 most significant bits
	cb = ( SrcP010[uvIndex] >> 6) - 512;
	cr = ( SrcP010[uvIndex + 1] >> 6) - 512;

	// calc BGR values
	b = 1.732446f * cb;
	g = -0.337633f * cb - 0.698001f * cr;
	r = 1.370705f * cr;

	// fetch and set each BGRA pixel
	float yp;
	uint32_t xOut, yOut;

#pragma unroll
	for (uint8_t j = 0; j < 2; j++)
	{
		yOut = y * 2 + j;
#pragma unroll
		for (uint8_t i = 0; i < 2; i++)
		{
			xOut = x * 2 + i;

			pixel = (Pitch / 2) * yOut + xOut;

			yp = SrcP010[pixel] >> 6;

			// ABGR
			uint32_t data = (0b11 << 30) | (min(uint32_t(yp + b), 1024) << 20) | (min(uint32_t(yp + g), 1024) << 10) | min(uint32_t(yp + r), 1024);

			surf2Dwrite(data, DstABGR10, xOut * sizeof(uint32_t), yOut);
		}
	}
}