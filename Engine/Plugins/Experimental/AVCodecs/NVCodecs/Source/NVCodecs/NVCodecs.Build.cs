// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using System;

public class NVCodecs : ModuleRules
{
	public NVCodecs(ReadOnlyTargetRules Target) : base(Target)
	{
		// Without these two compilation fails on VS2017 with D8049: command line is too long to fit in debug record.
		bLegacyPublicIncludePaths = false;
		DefaultBuildSettings = BuildSettingsVersion.Latest;

		PrivateDependencyModuleNames.AddRange(new string[] {
			"Engine",
			"AVCodecsCore",
			"CUDA"
		});

		PublicDependencyModuleNames.AddRange(new string[] {
			"RenderCore",
			"Core"
		});

		// Add CUDA kernels for colour space adaption
		PublicIncludePaths.Add(Path.Join(ModuleDirectory, "Kernels", "Public"));

		// TODO make some way to compile the kernels for easy testing
		/*nvcc -arch=sm_50 -gencode=arch=compute_50,code=sm_50 -gencode=arch=compute_52,code=sm_52 -gencode=arch=compute_60,code=sm_60 -gencode=arch=compute_61,code=sm_61 -gencode=arch=compute_70,code=sm_70 -gencode=arch=compute_75,code=sm_75 -gencode=arch=compute_75,code=compute_75 -o NV12_to_BGRA8.fatbin -fatbin --device-debug ../src/NV12_to_BGRA8.cu*/

		// RuntimeDependencies.Add(Path.Join("$(ProjectDir)", "Binaries", Target.Platform.ToString(), "Kernels", "NV12_to_BGRA8.fatbin"),
		// 						Path.Join(ModuleDirectory, "Kernels", "lib", "NV12_to_BGRA8.fatbin"));

		// RuntimeDependencies.Add(Path.Join("$(ProjectDir)", "Binaries", Target.Platform.ToString(), "Kernels", "P010_to_ABGR10.fatbin"),
		// 						Path.Join(ModuleDirectory, "Kernels", "lib", "P010_to_ABGR10.fatbin"));
		
		PrivateDependencyModuleNames.Add("Vulkan");
		AddEngineThirdPartyPrivateStaticDependencies(Target, "Vulkan");
		
		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");
		}
	}
}
