// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Misc/App.h"

#include "Video/Resources/VideoResourceCUDA.h"

class FNVCodecModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		if (FApp::CanEverRender())
		{
			FCUDAModule& Module = FModuleManager::LoadModuleChecked<FCUDAModule>("CUDA");
			
			Module.OnPostCUDAInit.AddLambda([]()
			{
				FAVDevice::GetHardwareDevice()->SetContext<FVideoContextCUDA>(
					MakeShared<FVideoContextCUDA>(
						FModuleManager::GetModuleChecked<FCUDAModule>("CUDA").GetCudaContext()));
			});
		}
	}
};

IMPLEMENT_MODULE(FNVCodecModule, NVCodecs);
