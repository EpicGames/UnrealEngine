// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Resources/VideoResourceCUDA.h"

#include "Video/Resources/Vulkan/VideoResourceVulkan.h"

#if PLATFORM_WINDOWS
#include "Video/Resources/D3D/VideoResourceD3D.h"
#endif

REGISTER_TYPEID(FVideoContextCUDA);
REGISTER_TYPEID(FVideoResourceCUDA);

TAVResult<CUarray_format> ConvertFormat(EVideoFormat Format)
{
	switch (Format)
	{
	case EVideoFormat::R8:
	case EVideoFormat::BGRA:
		return CU_AD_FORMAT_UNSIGNED_INT8;
	case EVideoFormat::G16:
		return CU_AD_FORMAT_UNSIGNED_INT16;
	default:
		return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("EVideoFormat format %d is not supported"), EnumToUnderlyingType(Format)), TEXT("CUDA"));
	}
}

FVideoContextCUDA::FVideoContextCUDA(CUcontext const& Raw)
	: Raw(Raw)
{
}

FVideoResourceCUDA::FVideoResourceCUDA(TSharedRef<FAVDevice> const& Device, CUarray Raw, FAVLayout const& Layout, FVideoDescriptor const& Descriptor)
	: TVideoResource(Device, Layout, Descriptor)
	, Raw(Raw)
{
}

FVideoResourceCUDA::FVideoResourceCUDA(TSharedRef<FAVDevice> const& Device, CUDA_EXTERNAL_MEMORY_HANDLE_DESC const& ExternalResourceDesc, CUDA_EXTERNAL_SEMAPHORE_HANDLE_DESC const& ExternalFenceDesc, FAVLayout const& Layout, FVideoDescriptor const& Descriptor, uint64 FenceValue)
	: TVideoResource(Device, Layout, Descriptor)
	, SemaphoreValue(FenceValue)
{
	FCUDAContextScope const ContextGuard(GetContext()->Raw);

	{
		CUresult Result = FCUDAModule::CUDA().cuImportExternalMemory(&ExternalArray, &ExternalResourceDesc);
		if (Result != CUDA_SUCCESS)
		{
			FAVResult::Log(EAVResult::ErrorMapping, TEXT("Failed to import external memory"), TEXT("CUDA"), Result);

			return;
		}
	}

	// HACK until all RHIs have CUDA capable fences we check to make sure that the fence has been passed in before importing it
	ExternalSemaphore = NULL;
	if (ExternalFenceDesc.handle.fd || ExternalFenceDesc.handle.win32.handle != nullptr)
	{
		CUresult Result = FCUDAModule::CUDA().cuImportExternalSemaphore(&ExternalSemaphore, &ExternalFenceDesc);
		if (Result != CUDA_SUCCESS)
		{
			FAVResult::Log(EAVResult::ErrorMapping, TEXT("Failed to import external semaphore"), TEXT("CUDA"), Result);

			return;
		}
	}

	{
		// Use raw descriptor information as we are directly mapping memory for transport to encoder
		CUDA_EXTERNAL_MEMORY_MIPMAPPED_ARRAY_DESC MipmapDesc = {};
		MipmapDesc.numLevels = 1; // Only support mip level 0
		MipmapDesc.offset = Layout.Offset;
		MipmapDesc.arrayDesc.Width = GetRawDescriptor().Width;
		MipmapDesc.arrayDesc.Height = GetRawDescriptor().Height;
		MipmapDesc.arrayDesc.Depth = 0;
		MipmapDesc.arrayDesc.NumChannels = GetRawDescriptor().GetNumChannels();
		MipmapDesc.arrayDesc.Format = ConvertFormat(GetRawDescriptor().Format);
		MipmapDesc.arrayDesc.Flags = CUDA_ARRAY3D_SURFACE_LDST; // CUDA_ARRAY3D_COLOR_ATTACHMENT;

		CUresult Result = FCUDAModule::CUDA().cuExternalMemoryGetMappedMipmappedArray(&MipArray, ExternalArray, &MipmapDesc);
		if (Result != CUDA_SUCCESS)
		{
			FAVResult::Log(EAVResult::ErrorMapping, TEXT("Failed to bind mipmappedArray"), TEXT("CUDA"), Result);

			return;
		}
	}

	{
		CUresult Result = FCUDAModule::CUDA().cuMipmappedArrayGetLevel(&MaxMipArray, MipArray, 0);
		if (Result != CUDA_SUCCESS)
		{
			FAVResult::Log(EAVResult::ErrorMapping, TEXT("Failed to bind to mip 0"), TEXT("CUDA"), Result);

			return;
		}
	}

	Raw = MaxMipArray;
}

FVideoResourceCUDA::FVideoResourceCUDA(TSharedRef<FAVDevice> const& Device, CUDA_EXTERNAL_MEMORY_HANDLE_DESC const& ExternalResourceDesc, CUDA_EXTERNAL_SEMAPHORE_HANDLE_DESC const& ExternalFenceDesc, TSharedRef<FVideoResource> const& ExternalResource, uint64 FenceValue)
	: FVideoResourceCUDA(Device, ExternalResourceDesc, ExternalFenceDesc, ExternalResource->GetLayout(), ExternalResource->GetDescriptor(), FenceValue)
{
}

FVideoResourceCUDA::~FVideoResourceCUDA()
{
	FCUDAContextScope const ContextGuard(GetContext()->Raw);

	if (MaxMipArray != nullptr)
	{
		CUresult const Result = FCUDAModule::CUDA().cuArrayDestroy(MaxMipArray);
		if (Result != CUDA_SUCCESS)
		{
			FAVResult::Log(EAVResult::ErrorUnmapping, TEXT("Failed to destroy array"), TEXT("CUDA"), Result);
		}
	}

	if (MipArray != nullptr)
	{
		CUresult const Result = FCUDAModule::CUDA().cuMipmappedArrayDestroy(MipArray);
		if (Result != CUDA_SUCCESS)
		{
			FAVResult::Log(EAVResult::ErrorUnmapping, TEXT("Failed to destroy mipmaps"), TEXT("CUDA"), Result);
		}
	}

	if (ExternalArray != nullptr)
	{
		CUresult const Result = FCUDAModule::CUDA().cuDestroyExternalMemory(ExternalArray);
		if (Result != CUDA_SUCCESS)
		{
			FAVResult::Log(EAVResult::ErrorUnmapping, TEXT("Failed to clean up external memory"), TEXT("CUDA"), Result);
		}
	}

	if (ExternalSemaphore != nullptr)
	{
		CUresult const Result = FCUDAModule::CUDA().cuDestroyExternalSemaphore(ExternalSemaphore);
		if (Result != CUDA_SUCCESS)
		{
			FAVResult::Log(EAVResult::ErrorUnmapping, TEXT("Failed to destroy external semaphore"), TEXT("CUDA"), Result);
		}
	}
}

FAVResult FVideoResourceCUDA::Validate() const
{
	if (Raw == nullptr)
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Raw resource is invalid"), TEXT("CUDA"));
	}

	return EAVResult::Success;
}

FAVResult FVideoResourceCUDA::ReadData(TArray64<uint8>& OutData) const
{
	FCUDAContextScope const ContextGuard(GetContext()->Raw);

	OutData.SetNumUninitialized(GetDescriptor().GetSizeInBytes());

	// Copy the texture
	{
		CUDA_MEMCPY2D CopyParams = {};
		CopyParams.srcMemoryType = CU_MEMORYTYPE_ARRAY;
		CopyParams.srcArray = Raw;
		CopyParams.dstMemoryType = CU_MEMORYTYPE_HOST;
		CopyParams.dstHost = OutData.GetData();
		CopyParams.WidthInBytes = GetRawDescriptor().Width * (GetDescriptor().Format == EVideoFormat::P010 ? 2 : 1);
		CopyParams.Height = GetRawDescriptor().Height;

		CUresult const Result = FCUDAModule::CUDA().cuMemcpy2D(&CopyParams);
		if (Result != CUDA_SUCCESS)
		{
			return FAVResult(EAVResult::Error, TEXT("Failed to copy host memory for read"), TEXT("CUDA"), Result);
		}
	}

	return EAVResult::Success;
}

FAVResult FVideoResourceCUDA::CopyFrom(CUdeviceptr Source, uint32 MapPitch)
{
	FCUDAContextScope const ContextGuard(GetContext()->Raw);

	{
		CUDA_MEMCPY2D CopyParams = CreateCopyFromParams(Source, MapPitch);

		CUresult const Result = FCUDAModule::CUDA().cuMemcpy2D(&CopyParams);
		if (Result != CUDA_SUCCESS)
		{
			return FAVResult(EAVResult::Error, TEXT("Failed to copy from device ptr to external memory"), TEXT("CUDA"), Result);
		}
	}

	return EAVResult::Success;
}

FAVResult FVideoResourceCUDA::CopyFromAsync(CUdeviceptr Source, uint32 MapPitch)
{
	FCUDAContextScope const ContextGuard(GetContext()->Raw);

	// TODO (aidan.possemiers) CUDA Module should be responsible for handing out streams and managing them
	CUstream cuStream;
	FCUDAModule::CUDA().cuStreamCreate(&cuStream, CU_STREAM_DEFAULT);

	{
		CUDA_MEMCPY2D CopyParams = CreateCopyFromParams(Source, MapPitch);

		CUresult const Result = FCUDAModule::CUDA().cuMemcpy2DAsync(&CopyParams, cuStream);
		if (Result != CUDA_SUCCESS)
		{
			return FAVResult(EAVResult::Error, TEXT("Failed to copy from device ptr to external memory"), TEXT("CUDA"), Result);
		}
	}

	// Signal the external fence
	if (ExternalSemaphore)
	{
		CUDA_EXTERNAL_SEMAPHORE_SIGNAL_PARAMS params = {};
		params.params.fence.value = SemaphoreValue + 1;

		FCUDAModule::CUDA().cuSignalExternalSemaphoresAsync(&ExternalSemaphore, &params, 1, cuStream);		
	}

	// Destroy the stream this will happen only when all tasks on the stream have completed
	{
		CUresult const Result = FCUDAModule::CUDA().cuStreamDestroy(cuStream);
		if (Result != CUDA_SUCCESS)
		{
			return FAVResult(EAVResult::Error, TEXT("Failed to destroy stream"), TEXT("CUDA"), Result);
		}
	}

	return EAVResult::Success;
}

FAVResult FVideoResourceCUDA::CopyTo(CUdeviceptr Target, uint32 MapPitch)
{
	FCUDAContextScope const ContextGuard(GetContext()->Raw);

	{
		CUDA_MEMCPY2D CopyParams = CreateCopyToParams(Target, MapPitch);

		CUresult const Result = FCUDAModule::CUDA().cuMemcpy2D(&CopyParams);
		if (Result != CUDA_SUCCESS)
		{
			return FAVResult(EAVResult::Error, TEXT("Failed to copy from external memory to device ptr"), TEXT("CUDA"), Result);
		}
	}

	return EAVResult::Success;
}

FAVResult FVideoResourceCUDA::CopyToAsync(CUdeviceptr Target, uint32 MapPitch)
{
	FCUDAContextScope const ContextGuard(GetContext()->Raw);

	// TODO (aidan.possemiers) CUDA Module should be responsible for handing out streams and managing them
	CUstream cuStream;
	FCUDAModule::CUDA().cuStreamCreate(&cuStream, CU_STREAM_DEFAULT);

	{
		CUDA_MEMCPY2D CopyParams = CreateCopyToParams(Target, MapPitch);

		CUresult const Result = FCUDAModule::CUDA().cuMemcpy2DAsync(&CopyParams, cuStream);
		if (Result != CUDA_SUCCESS)
		{
			return FAVResult(EAVResult::Error, TEXT("Failed to copy from external memory to device ptr"), TEXT("CUDA"), Result);
		}
	}

	// Signal the external fence
	if (ExternalSemaphore)
	{
		CUDA_EXTERNAL_SEMAPHORE_SIGNAL_PARAMS params = {};
		params.params.fence.value = SemaphoreValue + 1;

		FCUDAModule::CUDA().cuSignalExternalSemaphoresAsync(&ExternalSemaphore, &params, 1, cuStream);		
	}

	// Destroy the stream this will happen only when all tasks on the stream have completed
	{
		CUresult const Result = FCUDAModule::CUDA().cuStreamDestroy(cuStream);
		if (Result != CUDA_SUCCESS)
		{
			return FAVResult(EAVResult::Error, TEXT("Failed to destroy stream"), TEXT("CUDA"), Result);
		}
	}

	return EAVResult::Success;
}
#if PLATFORM_WINDOWS

template <>
DLLEXPORT FAVResult FAVExtension::TransformResource(TSharedPtr<FVideoResourceCUDA>& OutResource, TSharedPtr<FVideoResourceD3D11> const& InResource)
{
	if (InResource.IsValid())
	{
		if (InResource->GetDevice()->HasContext<FVideoContextCUDA>())
		{
			CUDA_EXTERNAL_MEMORY_HANDLE_DESC ExternalDesc = {};
			ExternalDesc.type = CU_EXTERNAL_MEMORY_HANDLE_TYPE_D3D11_RESOURCE_KMT;
			ExternalDesc.handle.win32.name = nullptr;
			ExternalDesc.handle.win32.handle = InResource->GetSharedHandle();
			ExternalDesc.size = InResource->GetOffset() + InResource->GetSize();
			ExternalDesc.flags |= CUDA_EXTERNAL_MEMORY_DEDICATED;

			// TODO (aidan.possemiers)
			CUDA_EXTERNAL_SEMAPHORE_HANDLE_DESC ExternalFenceDesc = {};

			OutResource = MakeShared<FVideoResourceCUDA>(InResource->GetDevice(), ExternalDesc, ExternalFenceDesc, InResource.ToSharedRef());

			return OutResource->Validate();
		}

		return FAVResult(EAVResult::ErrorMapping, TEXT("No CUDA context found"), TEXT("CUDA"));
	}

	return FAVResult(EAVResult::ErrorMapping, TEXT("Input resource is not valid"), TEXT("CUDA"));
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformResource(TSharedPtr<FVideoResourceCUDA>& OutResource, TSharedPtr<FVideoResourceD3D12> const& InResource)
{
	if (InResource.IsValid())
	{
		if (InResource->GetDevice()->HasContext<FVideoContextCUDA>())
		{
			// Depending on how we are running the engine the texture we are looking to bind could be
			// stored as part of a heap or as a commited resource 
			CUDA_EXTERNAL_MEMORY_HANDLE_DESC ExternalResouceDesc = {};
			if (InResource->GetHeap().IsValid())
			{
				ExternalResouceDesc.type = CU_EXTERNAL_MEMORY_HANDLE_TYPE_D3D12_HEAP;
				ExternalResouceDesc.handle.win32.name = nullptr;
				ExternalResouceDesc.handle.win32.handle = InResource->GetHeapSharedHandle();
				ExternalResouceDesc.size = static_cast<unsigned long long>(InResource->GetSizeInBytes());
				ExternalResouceDesc.flags |= CUDA_EXTERNAL_MEMORY_DEDICATED;
			}
			else
			{
				ExternalResouceDesc.type = CU_EXTERNAL_MEMORY_HANDLE_TYPE_D3D12_RESOURCE;
				ExternalResouceDesc.handle.win32.name = nullptr;
				ExternalResouceDesc.handle.win32.handle = InResource->GetResourceSharedHandle();
				ExternalResouceDesc.size = static_cast<unsigned long long>(InResource->GetSizeInBytes());
				ExternalResouceDesc.flags |= CUDA_EXTERNAL_MEMORY_DEDICATED;
			}

			CUDA_EXTERNAL_SEMAPHORE_HANDLE_DESC ExternalFenceDesc = {};
			ExternalFenceDesc.type = CU_EXTERNAL_SEMAPHORE_HANDLE_TYPE_D3D12_FENCE;
			ExternalFenceDesc.handle.win32.name = nullptr;
			ExternalFenceDesc.handle.win32.handle = InResource->GetFenceSharedHandle();

			OutResource = MakeShared<FVideoResourceCUDA>(InResource->GetDevice(), ExternalResouceDesc, ExternalFenceDesc, InResource.ToSharedRef(), InResource->GetFenceValue());
			
			return OutResource->Validate();
		}

		return FAVResult(EAVResult::ErrorMapping, TEXT("No CUDA context found"), TEXT("CUDA"));
	}

	return FAVResult(EAVResult::ErrorMapping, TEXT("Input resource is not valid"), TEXT("CUDA"));
}

#endif

template <>
DLLEXPORT FAVResult FAVExtension::TransformResource(TSharedPtr<FVideoResourceCUDA>& OutResource, TSharedPtr<FVideoResourceVulkan> const& InResource)
{
	if (InResource.IsValid())
	{
		if (InResource->GetDevice()->HasContext<FVideoContextCUDA>())
		{
			CUDA_EXTERNAL_MEMORY_HANDLE_DESC ExternalDesc = {};
			ExternalDesc.size = InResource->GetOffset() + InResource->GetMemorySize();

#if PLATFORM_WINDOWS
			ExternalDesc.type = CU_EXTERNAL_MEMORY_HANDLE_TYPE_OPAQUE_WIN32;
			ExternalDesc.handle.win32.name = nullptr;
			ExternalDesc.handle.win32.handle = InResource->GetSharedHandle();
#else
			ExternalDesc.type = CU_EXTERNAL_MEMORY_HANDLE_TYPE_OPAQUE_FD;
			ExternalDesc.handle.fd = InResource->GetSharedHandle();
#endif

			CUDA_EXTERNAL_SEMAPHORE_HANDLE_DESC ExternalFenceDesc = {};

#if PLATFORM_WINDOWS
			ExternalFenceDesc.type = CU_EXTERNAL_SEMAPHORE_HANDLE_TYPE_OPAQUE_WIN32;
			ExternalDesc.handle.win32.name = nullptr;
			ExternalFenceDesc.handle.win32.handle = InResource->GetFenceSharedHandle();
#else
			ExternalFenceDesc.type = CU_EXTERNAL_SEMAPHORE_HANDLE_TYPE_OPAQUE_FD;
			ExternalFenceDesc.handle.fd = InResource->GetFenceSharedHandle();
#endif

			OutResource = MakeShared<FVideoResourceCUDA>(InResource->GetDevice(), ExternalDesc, ExternalFenceDesc, InResource.ToSharedRef());

			return OutResource->Validate();
		}

		return FAVResult(EAVResult::ErrorMapping, TEXT("No CUDA context found"), TEXT("CUDA"));
	}

	return FAVResult(EAVResult::ErrorMapping, TEXT("Input resource is not valid"), TEXT("CUDA"));
}
