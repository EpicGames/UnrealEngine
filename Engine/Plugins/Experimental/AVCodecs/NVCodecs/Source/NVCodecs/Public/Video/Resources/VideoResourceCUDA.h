// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVContext.h"
#include "AVExtension.h"
#include "Video/VideoResource.h"

#include "CudaModule.h"

#define UE_API NVCODECS_API

struct FCUDAContextScope
{
	FCUDAContextScope(CUcontext Context)
	{
		FCUDAModule::CUDA().cuCtxPushCurrent(Context);
	}

	~FCUDAContextScope()
	{
		FCUDAModule::CUDA().cuCtxPopCurrent(nullptr);
	}
};

class FVideoContextCUDA : public FAVContext
{
public:
	CUcontext Raw;

	UE_API FVideoContextCUDA(CUcontext const& Raw);
};

class FVideoResourceCUDA : public TVideoResource<FVideoContextCUDA>
{
private:
	CUexternalSemaphore ExternalSemaphore = nullptr;
	uint64 SemaphoreValue = 0;
	CUexternalMemory ExternalArray = nullptr;
	CUmipmappedArray MipArray = nullptr;
	CUarray MaxMipArray = nullptr;

	CUarray Raw = nullptr;

public:
	inline CUarray GetRaw() const { return Raw; }

	UE_API FVideoResourceCUDA(TSharedRef<FAVDevice> const& Device, CUarray Raw, FAVLayout const& Layout, FVideoDescriptor const& Descriptor);
	UE_API FVideoResourceCUDA(TSharedRef<FAVDevice> const& Device, CUDA_EXTERNAL_MEMORY_HANDLE_DESC const& ExternalResourceDesc, CUDA_EXTERNAL_SEMAPHORE_HANDLE_DESC const& ExternalFenceDesc, FAVLayout const& Layout, FVideoDescriptor const& Descriptor, uint64 FenceValue = 0);
	UE_API FVideoResourceCUDA(TSharedRef<FAVDevice> const& Device, CUDA_EXTERNAL_MEMORY_HANDLE_DESC const& ExternalResourceDesc, CUDA_EXTERNAL_SEMAPHORE_HANDLE_DESC const& ExternalFenceDesc, TSharedRef<FVideoResource> const& ExternalResource, uint64 FenceValue = 0);
	UE_API virtual ~FVideoResourceCUDA() override;

	UE_API virtual FAVResult Validate() const override;

	UE_API FAVResult ReadData(TArray64<uint8>& OutData) const;

	UE_API FAVResult CopyFrom(CUdeviceptr Target, uint32 MapPitch);
	UE_API FAVResult CopyFromAsync(CUdeviceptr Target, uint32 MapPitch);

	UE_API FAVResult CopyTo(CUdeviceptr Source, uint32 MapPitch);
	UE_API FAVResult CopyToAsync(CUdeviceptr Source, uint32 MapPitch);

private:
	inline CUDA_MEMCPY2D CreateCopyFromParams(CUdeviceptr Source, uint32 SourcePitch)
	{
		CUDA_MEMCPY2D CopyParams = {};
		CopyParams.srcMemoryType = CU_MEMORYTYPE_DEVICE;
		CopyParams.srcDevice = Source;
		CopyParams.srcPitch = SourcePitch;
		CopyParams.dstMemoryType = CU_MEMORYTYPE_ARRAY;
		CopyParams.dstArray = Raw;
		CopyParams.WidthInBytes = GetRawDescriptor().Width * (GetDescriptor().Format == EVideoFormat::P010 ? 2 : 1);
		CopyParams.Height = GetRawDescriptor().Height;

		return CopyParams;
	}

	inline CUDA_MEMCPY2D CreateCopyToParams(CUdeviceptr Target, uint32 TargetPitch)
	{
		CUDA_MEMCPY2D CopyParams = {};
		CopyParams.srcMemoryType = CU_MEMORYTYPE_ARRAY;
		CopyParams.srcArray = Raw;
		CopyParams.dstMemoryType = CU_MEMORYTYPE_DEVICE;
		CopyParams.dstDevice = Target;
		CopyParams.dstPitch = TargetPitch;
		CopyParams.WidthInBytes = GetRawDescriptor().Width * (GetDescriptor().Format == EVideoFormat::P010 ? 2 : 1);
		CopyParams.Height = GetRawDescriptor().Height;

		return CopyParams;
	}
};

#if PLATFORM_WINDOWS

template <>
FAVResult FAVExtension::TransformResource(TSharedPtr<FVideoResourceCUDA>& OutResource, TSharedPtr<class FVideoResourceD3D11> const& InResource);

template <>
FAVResult FAVExtension::TransformResource(TSharedPtr<FVideoResourceCUDA>& OutResource, TSharedPtr<class FVideoResourceD3D12> const& InResource);

#endif

template <>
FAVResult FAVExtension::TransformResource(TSharedPtr<FVideoResourceCUDA>& OutResource, TSharedPtr<class FVideoResourceVulkan> const& InResource);

DECLARE_TYPEID(FVideoContextCUDA, NVCODECS_API);
DECLARE_TYPEID(FVideoResourceCUDA, NVCODECS_API);

#undef UE_API
