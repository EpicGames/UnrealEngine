// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using System;

public class NVCodecsRHI : ModuleRules
{
	public NVCodecsRHI(ReadOnlyTargetRules Target) : base(Target)
	{
		// Without these two compilation fails on VS2017 with D8049: command line is too long to fit in debug record.
		bLegacyPublicIncludePaths = false;
		DefaultBuildSettings = BuildSettingsVersion.Latest;

		PrivateDependencyModuleNames.AddRange(new string[] {
			"Core",
			"Engine",
			"AVCodecsCore",
			"RHI",
			"NVDEC",
			"NVENC",
		});
	}
}
