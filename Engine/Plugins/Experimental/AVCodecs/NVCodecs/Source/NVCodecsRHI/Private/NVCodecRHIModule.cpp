// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "RHI.h"
#include "AVResult.h"
#include "GenericPlatform/GenericPlatformDriver.h"

#include "NVDEC.h"
#include "NVENC.h"

class FNVCodecRHIModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		if (FApp::CanEverRender())
		{
			FCoreDelegates::GetOnPostEngineInit().AddLambda([]() {
				bool bExceedsMinimumDriverVersion = false;
				if (IsRHIDeviceNVIDIA())
				{
					FString RequiredDriverVersion;
#if PLATFORM_WINDOWS
					RequiredDriverVersion = TEXT("531.61");
#elif PLATFORM_LINUX
					RequiredDriverVersion = TEXT("530.41");
#endif

					bExceedsMinimumDriverVersion = FDriverVersion(GRHIAdapterUserDriverVersion) >= FDriverVersion(RequiredDriverVersion);
					if (!bExceedsMinimumDriverVersion)
					{
						FAVResult::Log(EAVResult::Error, FString::Printf(TEXT("Detected driver version (%s) is older than required (%s). Please update your drivers!"), *GRHIAdapterUserDriverVersion, *RequiredDriverVersion));
					}
				}

				const_cast<FNVDEC&>(FAPI::Get<FNVDEC>()).bHasCompatibleGPU = IsRHIDeviceNVIDIA() && bExceedsMinimumDriverVersion;
				const_cast<FNVENC&>(FAPI::Get<FNVENC>()).bHasCompatibleGPU = IsRHIDeviceNVIDIA() && bExceedsMinimumDriverVersion;
			});
		}
	}
};

IMPLEMENT_MODULE(FNVCodecRHIModule, NVCodecsRHI);
