// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using System;

public class NVDEC: ModuleRules
{
	public NVDEC(ReadOnlyTargetRules Target) : base(Target)
	{
		// Without these two compilation fails on VS2017 with D8049: command line is too long to fit in debug record.
		bLegacyPublicIncludePaths = false;
		DefaultBuildSettings = BuildSettingsVersion.Latest;

		PrivateDependencyModuleNames.AddRange(new string[] {
			"Engine",
			"AVCodecsCore",
			"VulkanRHI",
			"CUDA",
			"NVCodecs"
		});

		PublicDependencyModuleNames.AddRange(new string[] {
			"RenderCore",
			"Core",
			"RHI",
			"nvDecode"
		});

		AddEngineThirdPartyPrivateStaticDependencies(Target, "Vulkan");
		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11");
		}
	}
}
