// Copyright Epic Games, Inc. All Rights Reserved.

#include "NVDEC.h"

#include "CudaModule.h"
#include "HAL/PlatformProcess.h"

#include "AVResult.h"

REGISTER_TYPEID(FNVDEC);

#define LOAD_FROM_DLL(Name) \
	Name = (Name##Ptr)FPlatformProcess::GetDllExport(DllHandle, TEXT(#Name)); \
	if (Name == nullptr) \
	{ \
		bAllFunctionsLoaded = false; \
		FAVResult::Log(EAVResult::Warning, TEXT("Failed to get dll export function: " #Name), TEXT("NVDEC")); \
	}

#define LOAD_FROM_DLL64(Name) \
	Name = (Name##Ptr)FPlatformProcess::GetDllExport(DllHandle, TEXT(#Name) "64"); \
	if (Name == nullptr) \
	{ \
		bAllFunctionsLoaded = false; \
		FAVResult::Log(EAVResult::Warning, TEXT("Failed to get dll export function: " #Name), TEXT("NVDEC")); \
	}

FNVDEC::FNVDEC()
{
#if PLATFORM_WINDOWS
	// Early out on non-supported windows versions
	if (!FPlatformMisc::VerifyWindowsVersion(6, 2))
	{
		return;
	}
#endif

	// clear function call table
	FMemory::Memzero(static_cast<NV_DECODE_API_FUNCTION_LIST*>(this), sizeof(NV_DECODE_API_FUNCTION_LIST));

	// name of DLL/SO library
#if PLATFORM_WINDOWS
	static TCHAR const* DllName = TEXT("nvcuvid.dll");
#elif PLATFORM_LINUX
	static TCHAR const* DllName = TEXT("libnvcuvid.so.1");
#else
	static TCHAR const* DllName = nullptr;
#endif

	if (DllName != nullptr)
	{
		DllHandle = FPlatformProcess::GetDllHandle(DllName);
		if (DllHandle != nullptr)
		{
			bool bAllFunctionsLoaded = true;
			LOAD_FROM_DLL(cuvidCreateDecoder);
			LOAD_FROM_DLL(cuvidParseVideoData);
			LOAD_FROM_DLL(cuvidCreateVideoParser);
			LOAD_FROM_DLL(cuvidDestroyVideoParser);
			LOAD_FROM_DLL(cuvidCtxLockCreate);
			LOAD_FROM_DLL(cuvidCtxLockDestroy);
			LOAD_FROM_DLL(cuvidDecodePicture);
			LOAD_FROM_DLL(cuvidGetDecoderCaps);
			LOAD_FROM_DLL(cuvidGetDecodeStatus);
			LOAD_FROM_DLL(cuvidReconfigureDecoder);
			LOAD_FROM_DLL(cuvidDestroyDecoder);
			LOAD_FROM_DLL64(cuvidMapVideoFrame);
			LOAD_FROM_DLL64(cuvidUnmapVideoFrame);

			if (bAllFunctionsLoaded)
			{
				return;
			}

			FPlatformProcess::FreeDllHandle(DllHandle);
			DllHandle = nullptr;
		}
		else
		{
			FAVResult::Log(EAVResult::Unknown, TEXT("Failed to get NVDEC dll handle. NVDEC module will not be available."), TEXT("NVDEC"));
		}
	}
	else
	{
		FAVResult::Log(EAVResult::Warning, TEXT("Failed to get NVDEC dll name. NVDEC module will not be available."), TEXT("NVDEC"));
	}
}

bool FNVDEC::IsValid() const
{
	return DllHandle != nullptr && bHasCompatibleGPU;
}

FString FNVDEC::GetErrorString(CUresult ErrorCode) const
{
	char const* LastError = nullptr;

	FCUDAModule::CUDA().cuGetErrorString(ErrorCode, &LastError);

	if (LastError)
	{
		return FString::Format(TEXT("error {0} ('{1}')"), { ErrorCode, UTF8_TO_TCHAR(LastError) });
	}

	return FString("Error getting error string! Is CUDA configured correctly?");
}