// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Decoders/VideoDecoderNVDEC.h"
#include "Video/Decoders/Configs/VideoDecoderConfigAV1.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH264.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH265.h"
#include "Video/Resources/Vulkan/VideoResourceVulkan.h"

#if PLATFORM_WINDOWS
#include "Video/Resources/D3D/VideoResourceD3D.h"
#endif

class FNVDECModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		TFunction<bool(TSharedRef<FAVDevice> const&, cudaVideoCodec)> CheckCodecSupport = [](TSharedRef<FAVDevice> const& Device, cudaVideoCodec Codec) {
			if(!Device->HasContext<FVideoContextCUDA>() || !FAPI::Get<FNVDEC>().IsValid())
			{
				return false;
			}
			
			FCUDAContextScope const ContextGuard(Device->GetContext<FVideoContextCUDA>()->Raw);

			CUVIDDECODECAPS CapsToQuery;
			memset(&CapsToQuery, 0, sizeof(CapsToQuery));

			CapsToQuery.eCodecType = Codec;
			CapsToQuery.eChromaFormat = cudaVideoChromaFormat_420;
			// TODO (william.belcher): Should we query 10bit support?
			CapsToQuery.nBitDepthMinus8 = 0;

			CUresult const Result = FAPI::Get<FNVDEC>().cuvidGetDecoderCaps(&CapsToQuery);
			if (Result != CUDA_SUCCESS)
			{
				FAVResult::Log(EAVResult::Warning, TEXT("Failed to query for NVDEC capability"), TEXT("NVDEC"), Result);

				return false;
			}

			return CapsToQuery.bIsSupported > 0;
		};
		
		FVideoDecoder
			::RegisterPermutationsOf<FVideoDecoderNVDEC>
			::With<
				FVideoResourceCUDA,
#if PLATFORM_WINDOWS
				FVideoResourceD3D11, 
				FVideoResourceD3D12, 
#endif
				FVideoResourceVulkan>
			::And<FVideoDecoderConfigNVDEC, FVideoDecoderConfigH264>(
				[CheckCodecSupport](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					static bool bSupportsCodec = CheckCodecSupport(NewDevice, cudaVideoCodec_H264);

					return FAPI::Get<FNVDEC>().IsValid() && NewDevice->HasContext<FVideoContextCUDA>() && bSupportsCodec;
				});

		FVideoDecoder
			::RegisterPermutationsOf<FVideoDecoderNVDEC>
			::With<
				FVideoResourceCUDA,
#if PLATFORM_WINDOWS
				FVideoResourceD3D11, 
				FVideoResourceD3D12, 
#endif
				FVideoResourceVulkan>
			::And<FVideoDecoderConfigNVDEC, FVideoDecoderConfigH265>(
				[CheckCodecSupport](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					static bool bSupportsCodec = CheckCodecSupport(NewDevice, cudaVideoCodec_HEVC);

					return FAPI::Get<FNVDEC>().IsValid() && NewDevice->HasContext<FVideoContextCUDA>() && bSupportsCodec;
				});

		FVideoDecoder
			::RegisterPermutationsOf<FVideoDecoderNVDEC>
			::With<
				FVideoResourceCUDA,
#if PLATFORM_WINDOWS
				FVideoResourceD3D11, 
				FVideoResourceD3D12, 
#endif
				FVideoResourceVulkan>
			::And<FVideoDecoderConfigNVDEC, FVideoDecoderConfigAV1>(
				[CheckCodecSupport](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					static bool bSupportsCodec = CheckCodecSupport(NewDevice, cudaVideoCodec_AV1);

					return FAPI::Get<FNVDEC>().IsValid() && NewDevice->HasContext<FVideoContextCUDA>() && bSupportsCodec;
				});
	}
};

IMPLEMENT_MODULE(FNVDECModule, NVDEC);
