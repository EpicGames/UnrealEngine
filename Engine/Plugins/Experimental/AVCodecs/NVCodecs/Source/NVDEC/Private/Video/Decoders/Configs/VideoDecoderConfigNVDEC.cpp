// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Decoders/Configs/VideoDecoderConfigNVDEC.h"

REGISTER_TYPEID(FVideoDecoderConfigNVDEC);

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigNVDEC& OutConfig, FVideoDecoderConfig const& InConfig)
{
	OutConfig.Preset = InConfig.Preset;
	OutConfig.LatencyMode = InConfig.LatencyMode;

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfig& OutConfig, FVideoDecoderConfigNVDEC const& InConfig)
{
	OutConfig.Preset = InConfig.Preset;
	OutConfig.LatencyMode = InConfig.LatencyMode;
	
	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigNVDEC& OutConfig, FVideoDecoderConfigH264 const& InConfig)
{
	OutConfig.CodecType = cudaVideoCodec_H264;
	
	return FAVExtension::TransformConfig<FVideoDecoderConfigNVDEC, FVideoDecoderConfig>(OutConfig, InConfig);
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigNVDEC& OutConfig, FVideoDecoderConfigH265 const& InConfig)
{
	OutConfig.CodecType = cudaVideoCodec_HEVC;
	
	return FAVExtension::TransformConfig<FVideoDecoderConfigNVDEC, FVideoDecoderConfig>(OutConfig, InConfig);
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigNVDEC& OutConfig, FVideoDecoderConfigAV1 const& InConfig)
{
	OutConfig.CodecType = cudaVideoCodec_AV1;
	
	return FAVExtension::TransformConfig<FVideoDecoderConfigNVDEC, FVideoDecoderConfig>(OutConfig, InConfig);
}
