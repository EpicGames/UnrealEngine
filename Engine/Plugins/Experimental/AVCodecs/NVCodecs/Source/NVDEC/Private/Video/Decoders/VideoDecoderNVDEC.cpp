// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Decoders/VideoDecoderNVDEC.h"

#include "HAL/PlatformProcess.h"

#include "NV12_to_BGRA8.cuh"
#include "P010_to_ABGR10.cuh"

namespace Internal 
{
	int HandleVideoSequenceCallback(void* UserData, CUVIDEOFORMAT* VideoFormat)
	{
		return static_cast<FVideoDecoderNVDEC*>(UserData)->HandleVideoSequence(VideoFormat);
	}

	int HandlePictureDCodecallback(void *UserData, CUVIDPICPARAMS *PicParams)
	{
		return static_cast<FVideoDecoderNVDEC*>(UserData)->HandlePictureDecode(PicParams);
	}

	int HandlePictureDisplayCallback(void *UserData, CUVIDPARSERDISPINFO *DispInfo)
	{
		return static_cast<FVideoDecoderNVDEC*>(UserData)->HandlePictureDisplay(DispInfo);
	}
}

FVideoDecoderNVDEC::~FVideoDecoderNVDEC()
{
	Close();
}

bool FVideoDecoderNVDEC::IsOpen() const
{
	return bIsOpen;
}

FAVResult FVideoDecoderNVDEC::Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
{
	Close();

	TVideoDecoder::Open(NewDevice, NewInstance);

	bIsOpen = true;

	CUVIDPARSERPARAMS VideoParserParameters = {};
	FVideoDecoderConfigNVDEC const& PendingConfig = GetPendingConfig();
	VideoParserParameters.CodecType = PendingConfig.CodecType;
	VideoParserParameters.ulMaxNumDecodeSurfaces = 1;
	// TODO (william.belcher): bLowLatency ? 0 : 1;
	VideoParserParameters.ulMaxDisplayDelay = 0;
	VideoParserParameters.pUserData = this;
	VideoParserParameters.pfnSequenceCallback = Internal::HandleVideoSequenceCallback;
	VideoParserParameters.pfnDecodePicture = Internal::HandlePictureDCodecallback;
	VideoParserParameters.pfnDisplayPicture = Internal::HandlePictureDisplayCallback;

	if (FAPI::Get<FNVDEC>().cuvidCreateVideoParser(&Parser, &VideoParserParameters) != CUDA_SUCCESS)
	{
		return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to create parser"), TEXT("NVDEC"));
	}

	if (FAPI::Get<FNVDEC>().cuvidCtxLockCreate(&CtxLock, GetDevice()->GetContext<FVideoContextCUDA>()->Raw) != CUDA_SUCCESS)
	{
		return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to context lock decoder"), TEXT("NVDEC"));
	}

	return EAVResult::Success;
}

void FVideoDecoderNVDEC::Close()
{
	if (Decoder != nullptr)
	{
		FCUDAContextScope const ContextGuard(GetDevice()->GetContext<FVideoContextCUDA>()->Raw);

		CUresult const Result = FAPI::Get<FNVDEC>().cuvidDestroyDecoder(Decoder);
		if (Result != CUDA_SUCCESS)
		{
			FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to destroy NVDEC decoder"), TEXT("NVDEC"), Result);
		}

		Decoder = nullptr;
	}

	if (Parser != nullptr)
	{
		FCUDAContextScope const ContextGuard(GetDevice()->GetContext<FVideoContextCUDA>()->Raw);

		CUresult const Result = FAPI::Get<FNVDEC>().cuvidDestroyVideoParser(Parser);
		if (Result != CUDA_SUCCESS)
		{
			FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to destroy NVDEC parser"), TEXT("NVDEC"), Result);
		}
	}

	// We should only run this block if we've ever opened the decoder before
	// Basically just prevents this block from being run by the call to Close() in Open()
	if (IsOpen())
	{
		FCUDAContextScope const ContextGuard(GetDevice()->GetContext<FVideoContextCUDA>()->Raw);

		CUresult const Result = FAPI::Get<FNVDEC>().cuvidCtxLockDestroy(CtxLock);
		if (Result != CUDA_SUCCESS)
		{
			FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to destroy NVDEC context lock"), TEXT("NVDEC"), Result);
		}
	}

	bIsOpen = false;
}

bool FVideoDecoderNVDEC::IsInitialized() const
{
	return IsOpen() && Decoder != nullptr;
}

// ApplyConfig is different from ApplyConfig from other decoder classes in that it does not call Close on an error.
// This is because ApplyConfig is called from a function chain originating from cuvidParseVideoData.
// Because cuvidParseVideoData depends on the Parser object, ApplyConfig cannot call Close which destroys the Parser.
// Destroying Parser while in use will crash from within Nvidia dll.
FAVResult FVideoDecoderNVDEC::ApplyConfig()
{
	if (IsOpen())
	{
		FVideoDecoderConfigNVDEC const& PendingConfig = GetPendingConfig();
		if (AppliedConfig != PendingConfig)
		{
			if (IsInitialized())
			{
				// If any of these change, we need to re-initialization the decoder
				if (AppliedConfig.OutputFormat == PendingConfig.OutputFormat 
					&& AppliedConfig.ChromaFormat == PendingConfig.ChromaFormat)
				{
					bool bResChange = AppliedConfig.ulWidth != PendingConfig.ulWidth || AppliedConfig.ulHeight != PendingConfig.ulHeight;

					if(bResChange)
					{
						CUVIDRECONFIGUREDECODERINFO ReconfigParams = { 0 };

						ReconfigParams.ulWidth = PendingConfig.ulWidth;
						ReconfigParams.ulHeight = PendingConfig.ulHeight;

						ReconfigParams.display_area.bottom = PendingConfig.display_area.bottom;
						ReconfigParams.display_area.top = PendingConfig.display_area.top;
						ReconfigParams.display_area.left = PendingConfig.display_area.left;
						ReconfigParams.display_area.right = PendingConfig.display_area.right;
						ReconfigParams.ulTargetWidth = PendingConfig.ulWidth;
						ReconfigParams.ulTargetHeight = PendingConfig.ulHeight;
						ReconfigParams.ulNumDecodeSurfaces = PendingConfig.ulNumDecodeSurfaces;

						FCUDAContextScope const ContextGuard(GetDevice()->GetContext<FVideoContextCUDA>()->Raw);
						CUresult const Result = FAPI::Get<FNVDEC>().cuvidReconfigureDecoder(Decoder, &ReconfigParams);
						if (Result != CUDA_SUCCESS)
						{
							return FAVResult(EAVResult::Error, TEXT("Failed to reconfigure decoder"), TEXT("NVDEC"), Result);
						}
					}
				}
				else
				{
					FCUDAContextScope const ContextGuard(GetDevice()->GetContext<FVideoContextCUDA>()->Raw);
					CUresult const Result = FAPI::Get<FNVDEC>().cuvidDestroyDecoder(Decoder);
					if (Result != CUDA_SUCCESS)
					{
						return FAVResult(EAVResult::ErrorDestroying, TEXT("Failed to destroy NVDEC decoder"), TEXT("NVDEC"), Result);
					}

					Decoder = nullptr;
				}
			}

			if (!IsInitialized())
			{
				FCUDAContextScope const ContextGuard(GetDevice()->GetContext<FVideoContextCUDA>()->Raw);
				CUresult const Result = FAPI::Get<FNVDEC>().cuvidCreateDecoder(&Decoder, &const_cast<FVideoDecoderConfigNVDEC&>(PendingConfig));
				if (Result != CUDA_SUCCESS)
				{
					return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to create decoder"), TEXT("NVDEC"), Result);
				}
			}
		}

		return TVideoDecoder::ApplyConfig();
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("NVDEC"));
}

bool FVideoDecoderNVDEC::GetCapability(CUVIDDECODECAPS& CapsToQuery) const
{
	if (IsOpen())
	{
		FCUDAContextScope const ContextGuard(GetDevice()->GetContext<FVideoContextCUDA>()->Raw);

		CUresult const Result = FAPI::Get<FNVDEC>().cuvidGetDecoderCaps(&CapsToQuery);
		if (Result != CUDA_SUCCESS)
		{
			FAVResult::Log(EAVResult::Warning, TEXT("Failed to query for NVDEC capability"), TEXT("NVDEC"), Result);

			return false;
		}

		return CapsToQuery.bIsSupported > 0;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("NVDEC"));
}

FAVResult FVideoDecoderNVDEC::SendPacket(FVideoPacket const& Packet)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("NVDEC"));
	}

	CUVIDSOURCEDATAPACKET CUPacket = {0};
	CUPacket.payload = Packet.DataPtr.Get();
	CUPacket.payload_size = Packet.DataSize;
	CUPacket.flags = CUVID_PKT_TIMESTAMP;
	// TODO (william.belcher): Adding this flag assumes that each Packet contains a full frame, but is required
	// to prevent the parser from adding a 1 frame delay
	CUPacket.flags |= CUVID_PKT_ENDOFPICTURE;
	CUPacket.timestamp = 0;
	if (!Packet.DataPtr.Get() || Packet.DataSize == 0) 
	{
		CUPacket.flags |= CUVID_PKT_ENDOFSTREAM;
	}

	CUresult const CUResult = FAPI::Get<FNVDEC>().cuvidParseVideoData(Parser, &CUPacket);
	if (CUResult != CUDA_SUCCESS)
	{
		return FAVResult(EAVResult::Error, TEXT("Failed to parse video data"), TEXT("NVDEC"), CUResult);
	}

	return EAVResult::Success;
}

FAVResult FVideoDecoderNVDEC::ReceiveFrame(TResolvableVideoResource<FVideoResourceCUDA>& InOutResource)
{
	if (IsOpen())
	{
		FFrame Frame;
		if (Frames.Peek(Frame))
		{
			FCUDAContextScope const ContextGuard(GetDevice()->GetContext<FVideoContextCUDA>()->Raw);

			CUresult Result = CUDA_SUCCESS;
			CUVIDGETDECODESTATUS DecodeStatus = { };

			do
			{
				Result = FAPI::Get<FNVDEC>().cuvidGetDecodeStatus(Decoder, Frame.SurfaceIndex, &DecodeStatus);
				if (Result != CUDA_SUCCESS)
				{
					return FAVResult(EAVResult::Error, TEXT("Could not retrieve decoder status"), TEXT("NVDEC"));
				}
				FPlatformProcess::Sleep(0.5f / 1000.0f); // HACK (aidan.possemiers) surely we can avoid a busy wait here
			} while (DecodeStatus.decodeStatus == cuvidDecodeStatus_InProgress);

			if (DecodeStatus.decodeStatus < cuvidDecodeStatus_Success)
			{
				return FAVResult(EAVResult::PendingInput, TEXT("Send more frames"), TEXT("NVDEC"), DecodeStatus.decodeStatus);
			}

			Frames.Pop();
			--FramesCount;

			if (DecodeStatus.decodeStatus > cuvidDecodeStatus_Success)
			{
				FAVResult::Log(EAVResult::Warning, TEXT("Error when decoding frame"), TEXT("NVDEC"), DecodeStatus.decodeStatus);
			}

			// Map the decoder output buffer
			CUdeviceptr MapSurface = 0;
			uint32 MapPitch = 0;

			Result = FAPI::Get<FNVDEC>().cuvidMapVideoFrame(Decoder, Frame.SurfaceIndex, &MapSurface, &MapPitch, &Frame.MapParams);
			if (Result != CUDA_SUCCESS)
			{
				return FAVResult(EAVResult::ErrorMapping, TEXT("Failed to map decoded frame"), TEXT("NVDEC"), Result);
			}

			FVideoDescriptor ResourceDescriptor = {};
			switch (Frame.SurfaceFormat)
			{
				case cudaVideoSurfaceFormat_NV12:
					ResourceDescriptor = FVideoDescriptor(EVideoFormat::NV12, Frame.Width, Frame.Height);
					break;
				case cudaVideoSurfaceFormat_P016:
					ResourceDescriptor = FVideoDescriptor(EVideoFormat::P010, Frame.Width, Frame.Height);
					break;
				case cudaVideoSurfaceFormat_YUV444:
					ResourceDescriptor = FVideoDescriptor(EVideoFormat::YUV444, Frame.Width, Frame.Height);
					break;
				case cudaVideoSurfaceFormat_YUV444_16Bit:
					ResourceDescriptor = FVideoDescriptor(EVideoFormat::YUV444_16, Frame.Width, Frame.Height);
					break;
				default:
					return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("Currently do not support the recieved OutputFormat (%d)"), Frame.SurfaceFormat));
			}

			if (!InOutResource.Resolve(GetDevice(), ResourceDescriptor))
			{
				return FAVResult(EAVResult::ErrorResolving, TEXT("Failed to resolve frame resource"), TEXT("NVDEC"));
			}

			// Do copy into the VideoResource
			InOutResource->CopyFrom(MapSurface, MapPitch);

			// TODO (aidan.possemiers) this is messy as we may unmap the texture before the copy is done
			// from what I can see there is no way to easily unmap this one the copy is done
			Result = FAPI::Get<FNVDEC>().cuvidUnmapVideoFrame(Decoder, MapSurface);
			if (Result != CUDA_SUCCESS)
			{
				return FAVResult(EAVResult::ErrorUnmapping, TEXT("Failed to unmap decoded frame"), TEXT("NVDEC"), Result);
			}	

			return EAVResult::Success;
		}

		return EAVResult::PendingInput;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("NVDEC"));
}

int FVideoDecoderNVDEC::HandleVideoSequence(CUVIDEOFORMAT *VideoFormat)
{
	int NumDecodeSurfaces = VideoFormat->min_num_decode_surfaces;

	CUVIDDECODECAPS DecodeCaps;
	memset(&DecodeCaps, 0, sizeof(DecodeCaps));

	DecodeCaps.eCodecType = VideoFormat->codec;
	DecodeCaps.eChromaFormat = VideoFormat->chroma_format;
	DecodeCaps.nBitDepthMinus8 = VideoFormat->bit_depth_luma_minus8;

	if (!GetCapability(DecodeCaps))
	{
		FAVResult::Log(EAVResult::ErrorUnsupported, TEXT("Codec not supported on this GPU"), TEXT("NVDEC"));
		return NumDecodeSurfaces;
	}

	if ((VideoFormat->coded_width > DecodeCaps.nMaxWidth) || (VideoFormat->coded_height > DecodeCaps.nMaxHeight))
	{
		FAVResult::Log(EAVResult::ErrorUnsupported, FString::Printf(TEXT("Resolution: %dx%d\nMax Supported Resolution: %dx%d\n Resolution not supported on this GPU"), VideoFormat->coded_width, VideoFormat->coded_height, DecodeCaps.nMaxWidth, DecodeCaps.nMaxHeight), TEXT("NVDEC"));
		return NumDecodeSurfaces;
	}

	if ((VideoFormat->coded_width >> 4) * (VideoFormat->coded_height >> 4) > DecodeCaps.nMaxMBCount)
	{
		FAVResult::Log(EAVResult::ErrorUnsupported, FString::Printf(TEXT("MBCount: %d\nMax Supported mbcnt: %d\n MBCount not supported on this GPU"), (VideoFormat->coded_width >> 4) * (VideoFormat->coded_height >> 4), DecodeCaps.nMaxMBCount), TEXT("NVDEC"));
		return NumDecodeSurfaces;
	}

	FVideoDecoderConfigNVDEC& PendingConfig = EditPendingConfig();
	PendingConfig.CodecType = VideoFormat->codec;
	PendingConfig.ChromaFormat = VideoFormat->chroma_format;
	PendingConfig.OutputFormat = VideoFormat->bit_depth_luma_minus8 ? cudaVideoSurfaceFormat_P016 : cudaVideoSurfaceFormat_NV12;
	PendingConfig.bitDepthMinus8 = VideoFormat->bit_depth_luma_minus8;
	PendingConfig.DeinterlaceMode = cudaVideoDeinterlaceMode_Weave;
	PendingConfig.ulNumOutputSurfaces = 2;
	// With PreferCUVID, JPEG is still decoded by CUDA while video is decoded by NVDEC hardware
	PendingConfig.ulCreationFlags = cudaVideoCreate_PreferCUVID;
	PendingConfig.ulNumDecodeSurfaces = NumDecodeSurfaces;
	PendingConfig.vidLock = CtxLock;
	PendingConfig.ulWidth = VideoFormat->coded_width;
	PendingConfig.ulHeight = VideoFormat->coded_height;
	// TODO (Eden.Harris) By setting max width/height to the device max, the encoder may potentially be allocating multiple large frames in VRAM.
	// This may be alot when b-frames are enabled and should be monitored to make sure VRAM is not unnecessarily consumed.
	// The reason why this is set to the device max is because if the stream resizes, it cannot resize above this initial ulMaxWidth/ulMaxHeight.
	PendingConfig.ulMaxWidth = DecodeCaps.nMaxWidth;
	PendingConfig.ulMaxHeight = DecodeCaps.nMaxHeight;

	// TODO (william.belcher): Add support for cropping and resizing
	PendingConfig.ulTargetWidth = VideoFormat->coded_width;
	PendingConfig.ulTargetHeight = VideoFormat->coded_height;

	FAVResult Result = ApplyConfig();
	if (Result.IsNotSuccess())
	{
		FAVResult::Log(EAVResult::Error, TEXT("Failed to apply decoder config"), TEXT("NVDEC"), Result);
		return 0;	// Return fail. Unfortunately this does not bubble up and HandlePictureDecode will still be called.
	}

	return NumDecodeSurfaces;
}

int FVideoDecoderNVDEC::HandlePictureDecode(CUVIDPICPARAMS *PicParams)
{
	CUresult const Result = FAPI::Get<FNVDEC>().cuvidDecodePicture(Decoder, PicParams);
	if (Result != CUDA_SUCCESS)
	{
		FAVResult::Log(EAVResult::Error, TEXT("Failed to decode frame"), TEXT("NVDEC"), Result);
		return 0;	// Unfortunately this does not bubble up to cuvidParseVideoData
	}
	return 1;
}

int FVideoDecoderNVDEC::HandlePictureDisplay(CUVIDPARSERDISPINFO *DispInfo)
{
	CUVIDPROCPARAMS VideoProcessingParameters = {};
	VideoProcessingParameters.progressive_frame = DispInfo->progressive_frame;
	VideoProcessingParameters.second_field = DispInfo->repeat_first_field + 1;
	VideoProcessingParameters.top_field_first = DispInfo->top_field_first;
	VideoProcessingParameters.unpaired_field = DispInfo->repeat_first_field < 0;
	VideoProcessingParameters.output_stream = 0;

	Frames.Enqueue({ DispInfo->picture_index, AppliedConfig.ulTargetWidth, AppliedConfig.ulTargetHeight, GetAppliedConfig().OutputFormat, VideoProcessingParameters });
	++FramesCount;

	return 0;
}