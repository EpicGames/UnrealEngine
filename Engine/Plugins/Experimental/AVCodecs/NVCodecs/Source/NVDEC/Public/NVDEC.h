// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVUtility.h"

THIRD_PARTY_INCLUDES_START

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

#include <cuviddec.h>
#include <nvcuvid.h>

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_END

// Unlike NVENC, NVDEC (cuviddec) does not give us a nice struct to fill out at runtime so we will do it ourselves

// TODO (aidan) some of this includes a video stream parser we likely want to do this externally so that we can process our
// various stream types ourselves

struct NV_DECODE_API_FUNCTION_LIST
{
	typedef CUresult (*cuvidCreateDecoderPtr)(CUvideodecoder*, CUVIDDECODECREATEINFO*);
	typedef CUresult (*cuvidParseVideoDataPtr)(CUvideoparser, CUVIDSOURCEDATAPACKET*);
	typedef CUresult (*cuvidCreateVideoParserPtr)(CUvideoparser*, CUVIDPARSERPARAMS*);
	typedef CUresult (*cuvidDestroyVideoParserPtr)(CUvideoparser);
	typedef CUresult (*cuvidCtxLockCreatePtr)(CUvideoctxlock*, CUcontext);
	typedef CUresult (*cuvidCtxLockDestroyPtr)(CUvideoctxlock);
	typedef CUresult (*cuvidDecodePicturePtr)(CUvideodecoder, CUVIDPICPARAMS*);
	typedef CUresult (*cuvidGetDecoderCapsPtr)(CUVIDDECODECAPS*);
	typedef CUresult (*cuvidGetDecodeStatusPtr)(CUvideodecoder, int, CUVIDGETDECODESTATUS*);
	typedef CUresult (*cuvidReconfigureDecoderPtr)(CUvideodecoder, CUVIDRECONFIGUREDECODERINFO*);
	typedef CUresult (*cuvidDestroyDecoderPtr)(CUvideodecoder);
	typedef CUresult (*cuvidMapVideoFramePtr)(CUvideodecoder, int, CUdeviceptr*, unsigned int*, CUVIDPROCPARAMS*);
	typedef CUresult (*cuvidUnmapVideoFramePtr)(CUvideodecoder, CUdeviceptr);

	cuvidCreateDecoderPtr cuvidCreateDecoder = nullptr;
	cuvidParseVideoDataPtr cuvidParseVideoData = nullptr;
	cuvidCreateVideoParserPtr cuvidCreateVideoParser = nullptr;
	cuvidDestroyVideoParserPtr cuvidDestroyVideoParser = nullptr;
	cuvidCtxLockCreatePtr cuvidCtxLockCreate = nullptr;
	cuvidCtxLockDestroyPtr cuvidCtxLockDestroy = nullptr;
	cuvidDecodePicturePtr cuvidDecodePicture = nullptr;
	cuvidGetDecoderCapsPtr cuvidGetDecoderCaps = nullptr;
	cuvidGetDecodeStatusPtr cuvidGetDecodeStatus = nullptr;
	cuvidReconfigureDecoderPtr cuvidReconfigureDecoder = nullptr;
	cuvidDestroyDecoderPtr cuvidDestroyDecoder = nullptr;
	cuvidMapVideoFramePtr cuvidMapVideoFrame = nullptr;
	cuvidUnmapVideoFramePtr cuvidUnmapVideoFrame = nullptr;
};

class FNVDEC : public FAPI, public NV_DECODE_API_FUNCTION_LIST
{
public:
	bool bHasCompatibleGPU = true;

	NVDEC_API FNVDEC();

	NVDEC_API virtual bool IsValid() const override;

	NVDEC_API FString GetErrorString(CUresult ErrorCode) const;

private:
	void* DllHandle = nullptr;
};

DECLARE_TYPEID(FNVDEC, NVDEC_API);
