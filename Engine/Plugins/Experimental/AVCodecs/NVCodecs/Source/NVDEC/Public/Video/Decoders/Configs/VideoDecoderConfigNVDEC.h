// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVExtension.h"
#include "Video/VideoDecoder.h"
#include "Video/Decoders/Configs/VideoDecoderConfigAV1.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH264.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH265.h"

#include "NVDEC.h"

struct FVideoDecoderConfigNVDEC : public CUVIDDECODECREATEINFO, public FAVConfig
{
public:
	struct FParsedPicture : public CUVIDPICPARAMS
	{
	public:
		CUVIDDECODECREATEINFO DecodeCreateInfo;
		TArray<uint32> SliceOffsets;
	};
	
	FVideoDecoderConfigNVDEC(EAVPreset Preset = EAVPreset::Default)
		: CUVIDDECODECREATEINFO({})
		, FAVConfig(Preset)
	{
		// HACK (aidan) this can be parsed with H264
		ulMaxWidth = 4096;
		ulMaxHeight = 4096;
		
		// HACK (aidan) these are hard coded for now but should actually be determined from the output
		ulCreationFlags = cudaVideoCreate_PreferCUVID;
		// OutputFormat = cudaVideoSurfaceFormat_XXX; this is currently determined by parsing the BitStream
		ulNumOutputSurfaces = 1;
		ulTargetWidth = 1920;
		ulTargetHeight = 1080;
	}

	FVideoDecoderConfigNVDEC(FVideoDecoderConfigNVDEC const& Other)
	{
		*this = Other;
	}

	FVideoDecoderConfigNVDEC& operator=(FVideoDecoderConfigNVDEC const& Other)
	{
		FMemory::Memcpy(*this, Other);

		return *this;
	}

	bool operator==(FVideoDecoderConfigNVDEC const& Other) const
	{
		return FMemory::Memcmp(this, &Other, sizeof(FVideoDecoderConfigNVDEC)) == 0;
	}

	bool operator!=(FVideoDecoderConfigNVDEC const& Other) const
	{
		return !(*this == Other);
	}

	FAVResult Parse(TSharedRef<FAVInstance> const& Instance, FVideoPacket const& Packet, TArray<FParsedPicture>& OutPictures) { return EAVResult::Error; }

private:
	int RefPicIdx[16];
};

template <>
FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigNVDEC& OutConfig, struct FVideoDecoderConfig const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(struct FVideoDecoderConfig& OutConfig, FVideoDecoderConfigNVDEC const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigNVDEC& OutConfig, struct FVideoDecoderConfigH264 const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigNVDEC& OutConfig, struct FVideoDecoderConfigH265 const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoDecoderConfigNVDEC& OutConfig, struct FVideoDecoderConfigAV1 const& InConfig);

DECLARE_TYPEID(FVideoDecoderConfigNVDEC, NVDEC_API);
