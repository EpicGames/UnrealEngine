// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoDecoder.h"

#include "Containers/Queue.h"

#include "Video/Resources/VideoResourceCUDA.h"

#include "NVDEC.h"
#include "Video/Decoders/Configs/VideoDecoderConfigNVDEC.h"

#define UE_API NVDEC_API

class FVideoDecoderNVDEC : public TVideoDecoder<FVideoResourceCUDA, FVideoDecoderConfigNVDEC>
{
private:
	uint8 bIsOpen : 1;
	CUvideodecoder Decoder = nullptr;
	CUvideoparser Parser = nullptr;
	CUvideoctxlock CtxLock;

	struct FFrame
	{
		int32 SurfaceIndex;
		uint32 Width;
		uint32 Height;
		cudaVideoSurfaceFormat SurfaceFormat;
		
		CUVIDPROCPARAMS MapParams;
	};

	int32 FramesCount = 0;
	TQueue<FFrame> Frames;
	
public:
	FVideoDecoderNVDEC() = default;
	UE_API virtual ~FVideoDecoderNVDEC() override;

	UE_API virtual bool IsOpen() const override;
	UE_API virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override;
	UE_API virtual void Close() override;

	UE_API bool IsInitialized() const;

	UE_API virtual FAVResult ApplyConfig() override;

	UE_API bool GetCapability(CUVIDDECODECAPS& CapsToQuery) const;

	UE_API virtual FAVResult SendPacket(FVideoPacket const& Packet) override;
	
	UE_API virtual FAVResult ReceiveFrame(TResolvableVideoResource<FVideoResourceCUDA>& InOutResource) override;

public:
	UE_API int HandleVideoSequence(CUVIDEOFORMAT *VideoFormat);
    UE_API int HandlePictureDecode(CUVIDPICPARAMS *PicParams);
    UE_API int HandlePictureDisplay(CUVIDPARSERDISPINFO *DispInfo);
};

namespace Internal 
{
    int HandleVideoSequenceCallback(void *UserData, CUVIDEOFORMAT *VideoFormat);
	int HandlePictureDecodeCallback(void *UserData, CUVIDPICPARAMS *PicParams);
	int HandlePictureDisplayCallback(void *UserData, CUVIDPARSERDISPINFO *DispInfo);
}

#undef UE_API
