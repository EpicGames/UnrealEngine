// Copyright Epic Games, Inc. All Rights Reserved.

#include "NVENC.h"

#include "HAL/PlatformProcess.h"
#include "HAL/Thread.h"
#include "RHI.h"

#include "AVResult.h"

REGISTER_TYPEID(FNVENC);

// define a function pointer for creating an instance of nvEncodeAPI
typedef NVENCSTATUS(NVENCAPI* NVENCAPIPROC)(NV_ENCODE_API_FUNCTION_LIST*);

FNVENC::FNVENC()
{
#if PLATFORM_WINDOWS
	// Early out on non-supported windows versions
	if (!FPlatformMisc::VerifyWindowsVersion(6, 2))
	{
		return;
	}
#endif

	// clear function call table
	FMemory::Memzero(static_cast<NV_ENCODE_API_FUNCTION_LIST*>(this), sizeof(NV_ENCODE_API_FUNCTION_LIST));

	// name of DLL/SO library
#if PLATFORM_WINDOWS
	#if defined _WIN64
	static TCHAR const* DllName = TEXT("nvEncodeAPI64.dll");
	#else
	static TCHAR const* DllName = TEXT("nvEncodeAPI.dll");
	#endif // _WIN64
#elif PLATFORM_LINUX
	static TCHAR const* DllName = TEXT("libnvidia-encode.so.1");
#else
	static TCHAR const* DllName = nullptr;
#endif

	if (DllName != nullptr)
	{
		DllHandle = FPlatformProcess::GetDllHandle(DllName);

		if (DllHandle != nullptr)
		{
			if (NVENCAPIPROC NvEncodeAPICreateInstanceFunc = (NVENCAPIPROC)(FPlatformProcess::GetDllExport(DllHandle, TEXT("NvEncodeAPICreateInstance"))))
			{
				version = NV_ENCODE_API_FUNCTION_LIST_VER;

				if (NvEncodeAPICreateInstanceFunc(this) == NV_ENC_SUCCESS)
				{
					return;
				}
			}

			DllHandle = nullptr;
		}
		else
		{
			FAVResult::Log(EAVResult::Unknown, TEXT("Failed to get NVENC dll handle. NVENC module will not be available."), TEXT("NVENC"));
		}
	}
	else
	{
		FAVResult::Log(EAVResult::Warning, TEXT("Failed to get NVENC dll name. NVENC module will not be available."), TEXT("NVENC"));
	}
}

bool FNVENC::IsValid() const
{
	return DllHandle != nullptr && bHasCompatibleGPU;
}

FString FNVENC::GetErrorString(void* Encoder, NVENCSTATUS ForStatus) const
{
	char const* LastError = nvEncGetLastErrorString(Encoder);
	if (LastError)
	{
		return FString::Format(TEXT("error {0} ('{1}')"), { ForStatus, UTF8_TO_TCHAR(LastError) });
	}

	return FString("Error getting error string! Is NVENC configured correctly?");
}
