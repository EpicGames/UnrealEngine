// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/VideoEncoderNVENC.h"
#include "Video/Encoders/Configs/VideoEncoderConfigAV1.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH265.h"
#include "Video/Resources/Vulkan/VideoResourceVulkan.h"

#if PLATFORM_WINDOWS
	#include "Video/Resources/D3D/VideoResourceD3D.h"
#endif

#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"

namespace
{
	template <typename T> static constexpr bool TAlwaysFalse = false;

	template <typename TVideoContext>
	void* GetDeviceFromContext(TSharedPtr<TVideoContext> const& Context)
	{
		static_assert(TAlwaysFalse<TVideoContext>, "Unsupported context type");
		return nullptr;
	}

	template<>
	void* GetDeviceFromContext<FVideoContextCUDA>(TSharedPtr<FVideoContextCUDA> const& Context)
	{
		return Context->Raw;
	}

#if AVCODECS_USE_D3D
	template<>
	void* GetDeviceFromContext<FVideoContextD3D11>(TSharedPtr<FVideoContextD3D11> const& Context)
	{
		return Context->Device.GetReference();
	}

	template<>
	void* GetDeviceFromContext<FVideoContextD3D12>(TSharedPtr<FVideoContextD3D12> const& Context)
	{
		return Context->Device.GetReference();
	}
#endif
 
	template <typename TVideoContext>
	constexpr NV_ENC_DEVICE_TYPE GetNVENCDeviceType()
	{
		if constexpr (std::is_same_v<TVideoContext, FVideoContextCUDA>)
		{
			return NV_ENC_DEVICE_TYPE_CUDA;
		}
		else
		{
			return NV_ENC_DEVICE_TYPE_DIRECTX;
		}
	}

	template <typename TVideoContext>
	bool CheckCodecSupport(TSharedRef<FAVDevice> const& Device, GUID CodecGUID)
	{
		void* Encoder = nullptr;

		ON_SCOPE_EXIT
		{
			if (Encoder)
			{
				FAPI::Get<FNVENC>().nvEncDestroyEncoder(Encoder);
			}
		};

		if (!Device->HasContext<TVideoContext>())
		{
			return false;
		}

		if (!FAPI::Get<FNVENC>().IsValid())
		{
			return false;
		}

		NV_ENC_STRUCT(NV_ENC_OPEN_ENCODE_SESSION_EX_PARAMS, SessionParams);
		SessionParams.apiVersion = NVENCAPI_VERSION;
		SessionParams.deviceType = GetNVENCDeviceType<TVideoContext>();
		SessionParams.device = GetDeviceFromContext(Device->GetContext<TVideoContext>());
		NVENCSTATUS Result = FAPI::Get<FNVENC>().nvEncOpenEncodeSessionEx(&const_cast<NV_ENC_OPEN_ENCODE_SESSION_EX_PARAMS&>(SessionParams), &Encoder);
		if (Result != NV_ENC_SUCCESS)
		{
			return false;
		}

		uint32 Count = 0;
		Result = FAPI::Get<FNVENC>().nvEncGetEncodeGUIDCount(Encoder, &Count);
		if (Result != NV_ENC_SUCCESS || Count == 0)
		{
			return false;
		}

		TArray<GUID> GUIDs;
		GUIDs.SetNumZeroed(Count);
		Result = FAPI::Get<FNVENC>().nvEncGetEncodeGUIDs(Encoder, GUIDs.GetData(), Count, &Count);
		if (Result != NV_ENC_SUCCESS)
		{
			return false;
		}

		bool bSupported = false;
		for (uint32 i = 0; i < Count; i++)
		{
			if (!FMemory::Memcmp(&GUIDs[i], &CodecGUID, sizeof(GUID)))
			{
				bSupported = true;
				break;
			}
		}

		return bSupported;
	}
} // namespace

class FNVENCModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		/* Register NVENC+CUDA H264 pathway for Vulkan. */
		FVideoEncoder
			::RegisterPermutationsOf<FVideoEncoderNVENCCUDA>
			::With<
				FVideoResourceCUDA,
				FVideoResourceVulkan>
			::And<FVideoEncoderConfigNVENC, FVideoEncoderConfigH264>(
				[](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					// static local because we don't want to constantly create new encoder sessions every time we query codec support
					static bool bSupportsCodec = ::CheckCodecSupport<FVideoContextCUDA>(NewDevice, NV_ENC_CODEC_H264_GUID);

					return FAPI::Get<FNVENC>().IsValid() && NewDevice->HasContext<FVideoContextCUDA>() && bSupportsCodec;
				});

		/* Register NVENC+CUDA H265 pathway for Vulkan. */
		FVideoEncoder
			::RegisterPermutationsOf<FVideoEncoderNVENCCUDA>
			::With<
				FVideoResourceCUDA,
				FVideoResourceVulkan>
			::And<FVideoEncoderConfigNVENC, FVideoEncoderConfigH265>(
				[](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					// static local because we don't want to constantly create new encoder sessions every time we query codec support
					static bool bSupportsCodec = ::CheckCodecSupport<FVideoContextCUDA>(NewDevice, NV_ENC_CODEC_HEVC_GUID);

					return FAPI::Get<FNVENC>().IsValid() && NewDevice->HasContext<FVideoContextCUDA>() && bSupportsCodec;
				});

		/* Register NVENC+CUDA AV1 pathway for Vulkan. */
		FVideoEncoder
			::RegisterPermutationsOf<FVideoEncoderNVENCCUDA>
			::With<
				FVideoResourceCUDA,
				FVideoResourceVulkan>
			::And<FVideoEncoderConfigNVENC, FVideoEncoderConfigAV1>(
				[](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					// static local because we don't want to constantly create new encoder sessions every time we query codec support
					static bool bSupportsCodec = ::CheckCodecSupport<FVideoContextCUDA>(NewDevice, NV_ENC_CODEC_AV1_GUID);

					return FAPI::Get<FNVENC>().IsValid() && NewDevice->HasContext<FVideoContextCUDA>() && bSupportsCodec;
				});

		/* Register NVENC + raw DX11 graphics device pathway for D3D11. 
		* The reason we seperate these out in this way is because D3D11 + CUDA/NVENC does not encode UE textures properly due to how they are laid out.
		*/
#if AVCODECS_USE_D3D
		FVideoEncoder
			::RegisterPermutationsOf<FVideoEncoderNVENCD3D11>
			::With<FVideoResourceD3D11>
			::And<FVideoEncoderConfigNVENC, FVideoEncoderConfigH264>(
				[](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					// static local because we don't want to constantly create new encoder sessions every time we query codec support
					static bool bSupportsCodec = ::CheckCodecSupport<FVideoContextD3D11>(NewDevice, NV_ENC_CODEC_H264_GUID);

					return FAPI::Get<FNVENC>().IsValid() && NewDevice->HasContext<FVideoContextD3D11>() && bSupportsCodec;
				});

		FVideoEncoder
			::RegisterPermutationsOf<FVideoEncoderNVENCD3D11>
			::With<FVideoResourceD3D11>
			::And<FVideoEncoderConfigNVENC, FVideoEncoderConfigH265>(
				[](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					// static local because we don't want to constantly create new encoder sessions every time we query codec support
					static bool bSupportsCodec = ::CheckCodecSupport<FVideoContextD3D11>(NewDevice, NV_ENC_CODEC_HEVC_GUID);

					return FAPI::Get<FNVENC>().IsValid() && NewDevice->HasContext<FVideoContextD3D11>() && bSupportsCodec;
				});

		FVideoEncoder
			::RegisterPermutationsOf<FVideoEncoderNVENCD3D11>
			::With<FVideoResourceD3D11>
			::And<FVideoEncoderConfigNVENC, FVideoEncoderConfigAV1>(
				[](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					// static local because we don't want to constantly create new encoder sessions every time we query codec support
					static bool bSupportsCodec = ::CheckCodecSupport<FVideoContextD3D11>(NewDevice, NV_ENC_CODEC_AV1_GUID);

					return FAPI::Get<FNVENC>().IsValid() && NewDevice->HasContext<FVideoContextD3D11>() && bSupportsCodec;
				});


		static bool bD3D12UsesCUDA = false;
		GConfig->GetBool(TEXT("AVCodecs.NvEnc"), TEXT("D3D12UsesCUDA"), bD3D12UsesCUDA, GGameIni);
		FParse::Bool(FCommandLine::Get(), TEXT("AVCodecs.NvEnc.D3D12UsesCUDA="), bD3D12UsesCUDA);
		if (bD3D12UsesCUDA)
		{
			/* Register D3D12->CUDA->NVENC H264 pathway. */
			FVideoEncoder
				::RegisterPermutationsOf<FVideoEncoderNVENCCUDA>
				::With<
					FVideoResourceCUDA,
					FVideoResourceD3D12>
				::And<FVideoEncoderConfigNVENC, FVideoEncoderConfigH264>(
					[](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
					{
						// static local because we don't want to constantly create new encoder sessions every time we query codec support
						static bool bSupportsCodec = ::CheckCodecSupport<FVideoContextCUDA>(NewDevice, NV_ENC_CODEC_H264_GUID);

						return FAPI::Get<FNVENC>().IsValid() && NewDevice->HasContext<FVideoContextCUDA>() && bSupportsCodec;
					});

			/* Register D3D12->CUDA->NVENC H265 pathway. */
			FVideoEncoder
				::RegisterPermutationsOf<FVideoEncoderNVENCCUDA>
				::With<
					FVideoResourceCUDA,
					FVideoResourceD3D12>
				::And<FVideoEncoderConfigNVENC, FVideoEncoderConfigH265>(
					[](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
					{
						// static local because we don't want to constantly create new encoder sessions every time we query codec support
						static bool bSupportsCodec = ::CheckCodecSupport<FVideoContextCUDA>(NewDevice, NV_ENC_CODEC_HEVC_GUID);
					
						return FAPI::Get<FNVENC>().IsValid() && NewDevice->HasContext<FVideoContextCUDA>() && bSupportsCodec;
					});
				
			/* Register D3D12->CUDA->NVENC AV1 pathway. */
			FVideoEncoder
				::RegisterPermutationsOf<FVideoEncoderNVENCCUDA>
				::With<
					FVideoResourceCUDA,
					FVideoResourceD3D12>
				::And<FVideoEncoderConfigNVENC, FVideoEncoderConfigAV1>(
					[](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
					{
						// static local because we don't want to constantly create new encoder sessions every time we query codec support
						static bool bSupportsCodec = ::CheckCodecSupport<FVideoContextCUDA>(NewDevice, NV_ENC_CODEC_AV1_GUID);
					
						return FAPI::Get<FNVENC>().IsValid() && NewDevice->HasContext<FVideoContextCUDA>() && bSupportsCodec;
					});
		}
		else
		{
			/* Register NVENC + raw DX12 graphics device pathway for D3D12. 
			* The reason we seperate these out in this way is because D3D12 placed textures may not be correctly mapped to a cuda array.
			*/
			FVideoEncoder
			::RegisterPermutationsOf<FVideoEncoderNVENCD3D12>
			::With<FVideoResourceD3D12>
			::And<FVideoEncoderConfigNVENC, FVideoEncoderConfigH264>(
				[](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					// static local because we don't want to constantly create new encoder sessions every time we query codec support
					static bool bSupportsCodec = ::CheckCodecSupport<FVideoContextD3D12>(NewDevice, NV_ENC_CODEC_H264_GUID);

					return FAPI::Get<FNVENC>().IsValid() && NewDevice->HasContext<FVideoContextD3D12>() && bSupportsCodec;
				});
			
			FVideoEncoder
			::RegisterPermutationsOf<FVideoEncoderNVENCD3D12>
			::With<FVideoResourceD3D12>
			::And<FVideoEncoderConfigNVENC, FVideoEncoderConfigH265>(
				[](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					// static local because we don't want to constantly create new encoder sessions every time we query codec support
					static bool bSupportsCodec = ::CheckCodecSupport<FVideoContextD3D12>(NewDevice, NV_ENC_CODEC_HEVC_GUID);
					
					return FAPI::Get<FNVENC>().IsValid() && NewDevice->HasContext<FVideoContextD3D12>() && bSupportsCodec;
				});
				
			FVideoEncoder
			::RegisterPermutationsOf<FVideoEncoderNVENCD3D12>
			::With<FVideoResourceD3D12>
			::And<FVideoEncoderConfigNVENC, FVideoEncoderConfigAV1>(
				[](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
				{
					// static local because we don't want to constantly create new encoder sessions every time we query codec support
					static bool bSupportsCodec = ::CheckCodecSupport<FVideoContextD3D12>(NewDevice, NV_ENC_CODEC_AV1_GUID);
						
					return FAPI::Get<FNVENC>().IsValid() && NewDevice->HasContext<FVideoContextD3D12>() && bSupportsCodec;
				});
		}
#endif // AVCODECS_USE_D3D
	}
};

IMPLEMENT_MODULE(FNVENCModule, NVENC);
