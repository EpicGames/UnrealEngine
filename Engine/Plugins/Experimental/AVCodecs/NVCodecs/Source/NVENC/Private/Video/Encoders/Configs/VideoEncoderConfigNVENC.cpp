// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/Configs/VideoEncoderConfigNVENC.h"

#include "Video/Encoders/Configs/VideoEncoderConfigAV1.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH265.h"

REGISTER_TYPEID(FVideoEncoderConfigNVENC);

TAVResult<NV_ENC_PARAMS_RC_MODE> FVideoEncoderConfigNVENC::ConvertRateControlMode(ERateControlMode Mode)
{
	switch (Mode)
	{
	case ERateControlMode::ConstQP:
		return NV_ENC_PARAMS_RC_CONSTQP;
	case ERateControlMode::VBR:
		return NV_ENC_PARAMS_RC_VBR;
	case ERateControlMode::CBR:
		return NV_ENC_PARAMS_RC_CBR;
	default:
		return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("Rate control mode %d is not supported"), EnumToUnderlyingType(Mode)), TEXT("NVENC"));
	}
}

TAVResult<ERateControlMode> FVideoEncoderConfigNVENC::ConvertRateControlMode(NV_ENC_PARAMS_RC_MODE Mode)
{
	switch (Mode)
	{
	case NV_ENC_PARAMS_RC_CONSTQP:
		return ERateControlMode::ConstQP;
	case NV_ENC_PARAMS_RC_VBR:
		return ERateControlMode::VBR;
	case NV_ENC_PARAMS_RC_CBR:
		return ERateControlMode::CBR;
	default:
		return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("Rate control mode %d is not supported"), EnumToUnderlyingType(Mode)), TEXT("NVENC"));
	}
}

TAVResult<NV_ENC_MULTI_PASS> FVideoEncoderConfigNVENC::ConvertMultipassMode(EMultipassMode Mode)
{
	switch (Mode)
	{
	case EMultipassMode::Disabled:
		return NV_ENC_MULTI_PASS_DISABLED;
	case EMultipassMode::Quarter:
		return NV_ENC_TWO_PASS_QUARTER_RESOLUTION;
	case EMultipassMode::Full:
		return NV_ENC_TWO_PASS_FULL_RESOLUTION;
	default:
		return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("Multipass mode %d is not supported"), EnumToUnderlyingType(Mode)), TEXT("NVENC"));
	}
}

TAVResult<EMultipassMode> FVideoEncoderConfigNVENC::ConvertMultipassMode(NV_ENC_MULTI_PASS Mode)
{
	switch (Mode)
	{
	case NV_ENC_MULTI_PASS_DISABLED:
		return EMultipassMode::Disabled;
	case NV_ENC_TWO_PASS_QUARTER_RESOLUTION:
		return EMultipassMode::Quarter;
	case NV_ENC_TWO_PASS_FULL_RESOLUTION:
		return EMultipassMode::Full;
	default:
		return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("Multipass mode %d is not supported"), EnumToUnderlyingType(Mode)), TEXT("NVENC"));
	}
}

TAVResult<NV_ENC_BUFFER_FORMAT> FVideoEncoderConfigNVENC::ConvertFormat(EVideoFormat const& Format)
{
	switch (Format)
	{
	case EVideoFormat::BGRA:
		return NV_ENC_BUFFER_FORMAT::NV_ENC_BUFFER_FORMAT_ARGB;
	case EVideoFormat::ABGR10:
		return NV_ENC_BUFFER_FORMAT::NV_ENC_BUFFER_FORMAT_ARGB10;
	case EVideoFormat::NV12:
		return NV_ENC_BUFFER_FORMAT::NV_ENC_BUFFER_FORMAT_NV12;
	case EVideoFormat::P010:
		return NV_ENC_BUFFER_FORMAT::NV_ENC_BUFFER_FORMAT_YUV420_10BIT;
	default:
		return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("Pixel format %d is not supported"), EnumToUnderlyingType(Format)), TEXT("NVENC"));
	}

	// NV_ENC_BUFFER_FORMAT::NV_ENC_BUFFER_FORMAT_YUV420_10BIT : NV_ENC_BUFFER_FORMAT::NV_ENC_BUFFER_FORMAT_IYUV;
}

static uint32 DEFAULT_BITRATE_TARGET = 1000000;
static uint32 DEFAULT_BITRATE_MAX = 10000000;

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigNVENC& OutConfig, FVideoEncoderConfig const& InConfig)
{
	OutConfig.Preset = InConfig.Preset;
	OutConfig.LatencyMode = InConfig.LatencyMode;

	OutConfig.frameRateNum = InConfig.TargetFramerate > 0 ? InConfig.TargetFramerate : 60;
	OutConfig.encodeWidth = OutConfig.darWidth = InConfig.Width;
	OutConfig.maxEncodeWidth = FMath::Max(OutConfig.maxEncodeWidth, OutConfig.encodeWidth);
	OutConfig.encodeHeight = OutConfig.darHeight = InConfig.Height;
	OutConfig.maxEncodeHeight = FMath::Max(OutConfig.maxEncodeHeight, OutConfig.encodeHeight);

	switch (InConfig.Preset)
	{
	case EAVPreset::Lossless:
		OutConfig.tuningInfo = NV_ENC_TUNING_INFO_LOSSLESS;
		break;
	case EAVPreset::HighQuality:
		OutConfig.tuningInfo = NV_ENC_TUNING_INFO_HIGH_QUALITY;
		break;
	case EAVPreset::LowQuality:
		OutConfig.tuningInfo = NV_ENC_TUNING_INFO_LOW_LATENCY;
		break;
	default:
	case EAVPreset::UltraLowQuality:
		OutConfig.tuningInfo = NV_ENC_TUNING_INFO_ULTRA_LOW_LATENCY;
		break;
	}

	float QPRange = !memcmp(&OutConfig.encodeGUID, &NV_ENC_CODEC_AV1_GUID, sizeof(GUID)) ? 255.0f : 51.0f;
	uint32_t const MinQP = (1.0f - (InConfig.MaxQuality / 100.0f)) * QPRange;
	uint32_t const MaxQP = (1.0f - (InConfig.MinQuality / 100.0f)) * QPRange;

	ERateControlMode ActualRateControlMode = InConfig.Preset == EAVPreset::Lossless ? ERateControlMode::ConstQP : InConfig.RateControlMode;
	TAVResult<NV_ENC_PARAMS_RC_MODE> const ConvertedRateControlMode = FVideoEncoderConfigNVENC::ConvertRateControlMode(ActualRateControlMode);
	if (ConvertedRateControlMode.IsNotSuccess())
	{
		return ConvertedRateControlMode;
	}

	TAVResult<NV_ENC_MULTI_PASS> const ConvertedMultipassMode = FVideoEncoderConfigNVENC::ConvertMultipassMode(InConfig.MultipassMode);
	if (ConvertedMultipassMode.IsNotSuccess())
	{
		return ConvertedMultipassMode;
	}

	NV_ENC_RC_PARAMS& RateControlParams = OutConfig.encodeConfig->rcParams;
	RateControlParams.rateControlMode = ConvertedRateControlMode;
	RateControlParams.averageBitRate = InConfig.TargetBitrate > -1 ? InConfig.TargetBitrate : DEFAULT_BITRATE_TARGET;
	RateControlParams.maxBitRate = InConfig.MaxBitrate > -1 ? InConfig.MaxBitrate : DEFAULT_BITRATE_MAX; // Not used for CBR
	RateControlParams.multiPass = ConvertedMultipassMode;
	RateControlParams.minQP = { MinQP, MinQP, MinQP };
	RateControlParams.maxQP = { MaxQP, MaxQP, MaxQP };
	RateControlParams.constQP = RateControlParams.maxQP;
	RateControlParams.enableMinQP = InConfig.MaxQuality < 100;
	RateControlParams.enableMaxQP = InConfig.MinQuality > 0;
	RateControlParams.enableAQ = 1;
	if (ActualRateControlMode == ERateControlMode::CBR)
	{	
		RateControlParams.vbvBufferSize = RateControlParams.averageBitRate / (InConfig.TargetFramerate > 0 ? InConfig.TargetFramerate : 60);		
		RateControlParams.vbvInitialDelay = RateControlParams.vbvBufferSize;
	}

	// IDR period - how often to send IDR (instantaneous decode refresh) frames, a.k.a keyframes. This can stabilise a stream that dropped/lost some frames (but at the cost of more bandwidth)
	if (!FMemory::Memcmp(&OutConfig.encodeGUID, &NV_ENC_CODEC_H264_GUID, sizeof(GUID)))
	{
		if (InConfig.KeyframeInterval > 0)
		{
			OutConfig.encodeConfig->encodeCodecConfig.h264Config.idrPeriod = InConfig.KeyframeInterval;
		}
	}
	else if (!FMemory::Memcmp(&OutConfig.encodeGUID, &NV_ENC_CODEC_HEVC_GUID, sizeof(GUID)))
	{
		if (InConfig.KeyframeInterval > 0)
		{
			OutConfig.encodeConfig->encodeCodecConfig.hevcConfig.idrPeriod = InConfig.KeyframeInterval;
		}
	}
	else if (!FMemory::Memcmp(&OutConfig.encodeGUID, &NV_ENC_CODEC_AV1_GUID, sizeof(GUID)))
	{
		if (InConfig.KeyframeInterval > 0)
		{
			OutConfig.encodeConfig->encodeCodecConfig.av1Config.idrPeriod = InConfig.KeyframeInterval;
		}
	}

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfig& OutConfig, FVideoEncoderConfigNVENC const& InConfig)
{
	OutConfig.Preset = InConfig.Preset;
	OutConfig.LatencyMode = InConfig.LatencyMode;

	OutConfig.TargetFramerate = InConfig.frameRateNum;
	OutConfig.Width = InConfig.encodeWidth;
	OutConfig.Height = InConfig.encodeHeight;

	NV_ENC_RC_PARAMS& RateControlParams = InConfig.encodeConfig->rcParams;
	float QPRange = !memcmp(&InConfig.encodeGUID, &NV_ENC_CODEC_AV1_GUID, sizeof(GUID)) ? 255.0f : 51.0f;
	OutConfig.MinQuality = static_cast<int32>((1.0f - (RateControlParams.maxQP.qpIntra / QPRange)) * 100.0f);
	OutConfig.MaxQuality = static_cast<int32>((1.0f - (RateControlParams.minQP.qpIntra / QPRange)) * 100.0f);
	OutConfig.TargetBitrate = RateControlParams.averageBitRate == DEFAULT_BITRATE_TARGET ? -1 : RateControlParams.averageBitRate;
	OutConfig.MaxBitrate = RateControlParams.maxBitRate == DEFAULT_BITRATE_MAX ? -1 : RateControlParams.maxBitRate;

	TAVResult<ERateControlMode> const ConvertedRateControlMode = FVideoEncoderConfigNVENC::ConvertRateControlMode(RateControlParams.rateControlMode);
	if (ConvertedRateControlMode.IsNotSuccess())
	{
		return ConvertedRateControlMode;
	}

	OutConfig.RateControlMode = ConvertedRateControlMode;

	TAVResult<EMultipassMode> const ConvertedMultipassMode = FVideoEncoderConfigNVENC::ConvertMultipassMode(RateControlParams.multiPass);
	if (ConvertedMultipassMode.IsNotSuccess())
	{
		return ConvertedMultipassMode;
	}

	OutConfig.MultipassMode = ConvertedMultipassMode;

	if (!FMemory::Memcmp(&InConfig.encodeGUID, &NV_ENC_CODEC_H264_GUID, sizeof(GUID)))
	{
		OutConfig.KeyframeInterval = InConfig.encodeConfig->encodeCodecConfig.h264Config.idrPeriod;
	}
	else if (!FMemory::Memcmp(&InConfig.encodeGUID, &NV_ENC_CODEC_HEVC_GUID, sizeof(GUID)))
	{
		OutConfig.KeyframeInterval = InConfig.encodeConfig->encodeCodecConfig.hevcConfig.idrPeriod;
	}

	return EAVResult::Success;
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigNVENC& OutConfig, FVideoEncoderConfigH264 const& InConfig)
{
	// Unused from preset
	/*Config.encodeCodecConfig.h264Config.h264VUIParameters.videoFormat = 5;
	Config.encodeCodecConfig.h264Config.h264VUIParameters.colourPrimaries = 2;
	Config.encodeCodecConfig.h264Config.h264VUIParameters.transferCharacteristics = 2;
	Config.encodeCodecConfig.h264Config.h264VUIParameters.colourMatrix = 2;*/

	static auto const ConvertProfile = [](EH264Profile Profile) -> TAVResult<GUID> {
		switch (Profile)
		{
		case EH264Profile::Auto:
			return NV_ENC_CODEC_PROFILE_AUTOSELECT_GUID;
		case EH264Profile::Baseline:
			return NV_ENC_H264_PROFILE_BASELINE_GUID;
		case EH264Profile::Main:
			return NV_ENC_H264_PROFILE_MAIN_GUID;
		case EH264Profile::High:
			return NV_ENC_H264_PROFILE_HIGH_GUID;
		case EH264Profile::ProgressiveHigh:
			return NV_ENC_H264_PROFILE_PROGRESSIVE_HIGH_GUID;
		case EH264Profile::ConstrainedHigh:
			return NV_ENC_H264_PROFILE_CONSTRAINED_HIGH_GUID;
		case EH264Profile::High10:
			return NV_ENC_HEVC_PROFILE_MAIN10_GUID;
		case EH264Profile::StereoHigh:
			return NV_ENC_H264_PROFILE_STEREO_GUID;
		case EH264Profile::High444:
			return NV_ENC_H264_PROFILE_HIGH_444_GUID;
		default:
			return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("H264 profile %d is not supported"), EnumToUnderlyingType(Profile)), TEXT("NVENC"));
		}
		};

	OutConfig.encodeGUID = NV_ENC_CODEC_H264_GUID;

	TAVResult<GUID> const ConvertedProfileGUID = ConvertProfile(InConfig.Profile);
	if (ConvertedProfileGUID.IsNotSuccess())
	{
		return ConvertedProfileGUID;
	}

	OutConfig.encodeConfig->profileGUID = ConvertedProfileGUID;
	OutConfig.encodeConfig->rcParams.version = NV_ENC_RC_PARAMS_VER;

	NV_ENC_CONFIG_H264& OutH264Config = OutConfig.encodeConfig->encodeCodecConfig.h264Config;
	OutH264Config.chromaFormatIDC = 1;
	OutH264Config.enableFillerDataInsertion = InConfig.bFillData ? 1 : 0;

	static auto const ConvertAdaptiveTransformMode = [](EH264AdaptiveTransformMode TransformMode) -> TAVResult<NV_ENC_H264_ADAPTIVE_TRANSFORM_MODE> {
		switch (TransformMode)
		{
		case EH264AdaptiveTransformMode::Auto:
			return NV_ENC_H264_ADAPTIVE_TRANSFORM_AUTOSELECT;
		case EH264AdaptiveTransformMode::Disable:
			return NV_ENC_H264_ADAPTIVE_TRANSFORM_DISABLE;
		case EH264AdaptiveTransformMode::Enable:
			return NV_ENC_H264_ADAPTIVE_TRANSFORM_ENABLE;
		default:
			return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("H264 transform mode %d is not supported"), EnumToUnderlyingType(TransformMode)), TEXT("NVENC"));
		}
		};

	TAVResult<NV_ENC_H264_ADAPTIVE_TRANSFORM_MODE> const ConvertedAdaptiveTransformMode = ConvertAdaptiveTransformMode(InConfig.AdaptiveTransformMode);
	if (ConvertedAdaptiveTransformMode.IsNotSuccess())
	{
		return ConvertedAdaptiveTransformMode;
	}

	OutH264Config.adaptiveTransformMode = ConvertedAdaptiveTransformMode;

	static auto const ConvertEntropyCodingMode = [](EH264EntropyCodingMode EntropyCodingMode) -> TAVResult<NV_ENC_H264_ENTROPY_CODING_MODE> {
		switch (EntropyCodingMode)
		{
		case EH264EntropyCodingMode::Auto:
			return NV_ENC_H264_ENTROPY_CODING_MODE_AUTOSELECT;
		case EH264EntropyCodingMode::CABAC:
			return NV_ENC_H264_ENTROPY_CODING_MODE_CABAC;
		case EH264EntropyCodingMode::CAVLC:
			return NV_ENC_H264_ENTROPY_CODING_MODE_CAVLC;
		default:
			return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("H264 entropy coding mode %d is not supported"), EnumToUnderlyingType(EntropyCodingMode)), TEXT("NVENC"));
		}
		};

	TAVResult<NV_ENC_H264_ENTROPY_CODING_MODE> const ConvertedEntropyCodingMode = ConvertEntropyCodingMode(InConfig.EntropyCodingMode);
	if (ConvertedEntropyCodingMode.IsNotSuccess())
	{
		return ConvertedEntropyCodingMode;
	}

	OutH264Config.entropyCodingMode = ConvertedEntropyCodingMode;

	if (InConfig.RateControlMode == ERateControlMode::CBR && InConfig.bFillData)
	{
		// `outputPictureTimingSEI` is used in CBR mode to fill video frame with data to match the requested bitrate.
		OutH264Config.outputPictureTimingSEI = 1;
	}

	// Repeat SPS/PPS - sends sequence and picture parameter info with every IDR frame - maximum stabilisation of the stream when IDR is sent.
	OutH264Config.repeatSPSPPS = InConfig.RepeatSPSPPS;

	// Slice mode - set the slice mode to "entire frame as a single slice" because WebRTC implementation doesn't work well with slicing. The default slicing mode
	// produces (rarely, but especially under packet loss) grey full screen or just top half of it.
	OutH264Config.sliceMode = 0;
	OutH264Config.sliceModeData = 0;

	// These put extra meta data into the frames that allow Firefox to join mid stream.
	OutH264Config.outputFramePackingSEI = 1;
	OutH264Config.outputRecoveryPointSEI = 1;

	// Intra refresh - used to stabilise stream on the decoded side when frames are dropped/lost.
	if (InConfig.IntraRefreshPeriodFrames > 0)
	{
		OutH264Config.enableIntraRefresh = 1;
		OutH264Config.intraRefreshPeriod = InConfig.IntraRefreshPeriodFrames;
		OutH264Config.intraRefreshCnt = InConfig.IntraRefreshCountFrames;
	}

	return FAVExtension::TransformConfig<FVideoEncoderConfigNVENC, FVideoEncoderConfig>(OutConfig, InConfig);
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigNVENC& OutConfig, FVideoEncoderConfigH265 const& InConfig)
{
	// Unused from preset
	/*Config.encodeCodecConfig.hevcConfig.maxNumRefFramesInDPB = 2;
	Config.encodeCodecConfig.hevcConfig.hevcVUIParameters.videoFormat = 5;
	Config.encodeCodecConfig.hevcConfig.hevcVUIParameters.colourPrimaries = 2;
	Config.encodeCodecConfig.hevcConfig.hevcVUIParameters.transferCharacteristics = 2;
	Config.encodeCodecConfig.hevcConfig.hevcVUIParameters.colourMatrix = 2;*/

	static auto const ConvertProfile = [](EH265Profile Profile) -> TAVResult<GUID> {
		switch (Profile)
		{
		case EH265Profile::Auto:
		case EH265Profile::Main:
			return NV_ENC_HEVC_PROFILE_MAIN_GUID;
		case EH265Profile::Main10:
			return NV_ENC_HEVC_PROFILE_MAIN10_GUID;
		case EH265Profile::Main12:
			return NV_ENC_HEVC_PROFILE_FREXT_GUID;
		default:
			return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("H265 profile %d is not supported"), EnumToUnderlyingType(Profile)), TEXT("NVENC"));
		}
		};

	OutConfig.encodeGUID = NV_ENC_CODEC_HEVC_GUID;

	TAVResult<GUID> const ConvertedProfileGUID = ConvertProfile(InConfig.Profile);
	if (ConvertedProfileGUID.IsNotSuccess())
	{
		return ConvertedProfileGUID;
	}

	OutConfig.encodeConfig->profileGUID = ConvertedProfileGUID;
	OutConfig.encodeConfig->rcParams.version = NV_ENC_RC_PARAMS_VER;

	NV_ENC_CONFIG_HEVC& OutH265Config = OutConfig.encodeConfig->encodeCodecConfig.hevcConfig;
	OutH265Config.chromaFormatIDC = 1;
	OutH265Config.enableFillerDataInsertion = InConfig.bFillData ? 1 : 0;

	if (InConfig.RateControlMode == ERateControlMode::CBR && InConfig.bFillData)
	{
		// `outputPictureTimingSEI` is used in CBR mode to fill video frame with data to match the requested bitrate.
		OutH265Config.outputPictureTimingSEI = 1;
	}

	// Repeat SPS/PPS - sends sequence and picture parameter info with every IDR frame - maximum stabilisation of the stream when IDR is sent.
	OutH265Config.repeatSPSPPS = InConfig.RepeatSPSPPS;

	// Slice mode - set the slice mode to "entire frame as a single slice" because WebRTC implementation doesn't work well with slicing. The default slicing mode
	// produces (rarely, but especially under packet loss) grey full screen or just top half of it.
	OutH265Config.sliceMode = 0;
	OutH265Config.sliceModeData = 0;

	switch (InConfig.Profile)
	{
	case EH265Profile::Main:
	default:
		OutH265Config.pixelBitDepthMinus8 = 0;
		break;
	case EH265Profile::Main10:
		OutH265Config.pixelBitDepthMinus8 = 2;
		break;
	case EH265Profile::Main12:
		OutH265Config.pixelBitDepthMinus8 = 4;
		break;
	}

	// Intra refresh - used to stabilise stream on the decoded side when frames are dropped/lost.
	if (InConfig.IntraRefreshPeriodFrames > 0)
	{
		OutH265Config.enableIntraRefresh = 1;
		OutH265Config.intraRefreshPeriod = InConfig.IntraRefreshPeriodFrames;
		OutH265Config.intraRefreshCnt = InConfig.IntraRefreshCountFrames;
	}

	return FAVExtension::TransformConfig<FVideoEncoderConfigNVENC, FVideoEncoderConfig>(OutConfig, InConfig);
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigNVENC& OutConfig, FVideoEncoderConfigAV1 const& InConfig)
{
	static auto const ConvertProfile = [](EAV1Profile Profile) -> TAVResult<GUID> {
		switch (Profile)
		{
		case EAV1Profile::Auto:
		case EAV1Profile::Main:
			return NV_ENC_AV1_PROFILE_MAIN_GUID;
		default:
			return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("AV1 profile %d is not supported"), EnumToUnderlyingType(Profile)), TEXT("NVENC"));
		}
		};

	OutConfig.encodeGUID = NV_ENC_CODEC_AV1_GUID;

	TAVResult<GUID> const ConvertedProfileGUID = ConvertProfile(InConfig.Profile);
	if (ConvertedProfileGUID.IsNotSuccess())
	{
		return ConvertedProfileGUID;
	}

	OutConfig.encodeConfig->profileGUID = ConvertedProfileGUID;
	OutConfig.encodeConfig->rcParams.version = NV_ENC_RC_PARAMS_VER;

	NV_ENC_CONFIG_AV1& OutAV1Config = OutConfig.encodeConfig->encodeCodecConfig.av1Config;
	OutAV1Config.chromaFormatIDC = 1;

	if (InConfig.RateControlMode == ERateControlMode::CBR && InConfig.bFillData)
	{
		OutAV1Config.enableBitstreamPadding = InConfig.bFillData ? 1 : 0;
	}

	// Repeat sequence header - sends sequence header with every IDR frame - maximum stabilisation of the stream when IDR is sent.
	OutAV1Config.repeatSeqHdr = InConfig.RepeatSeqHdr;

	OutAV1Config.level = NV_ENC_LEVEL_AV1_AUTOSELECT;
	OutAV1Config.colorPrimaries = NV_ENC_VUI_COLOR_PRIMARIES_UNSPECIFIED;
	OutAV1Config.transferCharacteristics = NV_ENC_VUI_TRANSFER_CHARACTERISTIC_UNSPECIFIED;
	OutAV1Config.matrixCoefficients = NV_ENC_VUI_MATRIX_COEFFS_UNSPECIFIED;

	switch (InConfig.Profile)
	{
	case EAV1Profile::Auto:
	case EAV1Profile::Main:
	default:
		OutAV1Config.pixelBitDepthMinus8 = 0;
		break;
	}

	// Intra refresh - used to stabilise stream on the decoded side when frames are dropped/lost.
	if (InConfig.IntraRefreshPeriodFrames > 0)
	{
		OutAV1Config.enableIntraRefresh = 1;
		OutAV1Config.intraRefreshPeriod = InConfig.IntraRefreshPeriodFrames;
		OutAV1Config.intraRefreshCnt = InConfig.IntraRefreshCountFrames;
	}

	return FAVExtension::TransformConfig<FVideoEncoderConfigNVENC, FVideoEncoderConfig>(OutConfig, InConfig);
}
