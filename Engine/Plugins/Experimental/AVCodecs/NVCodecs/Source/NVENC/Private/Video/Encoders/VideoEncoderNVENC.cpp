// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/VideoEncoderNVENC.h"
#include "Templates/AlignmentTemplates.h"

FAVResult FEncoderNVENC::Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance, NV_ENC_OPEN_ENCODE_SESSION_EX_PARAMS& SessionParams)
{
	Close();

	SessionDeviceType = SessionParams.deviceType;
	SessionDevice = SessionParams.device;

	NVENCSTATUS const Result = FAPI::Get<FNVENC>().nvEncOpenEncodeSessionEx(&const_cast<NV_ENC_OPEN_ENCODE_SESSION_EX_PARAMS&>(SessionParams), &Encoder);
	if (Result != NV_ENC_SUCCESS)
	{
		Close();

		return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to create encoder"), TEXT("NVENC"), Result);
	}

	return EAVResult::Success;
}

FAVResult FEncoderNVENC::ReOpen()
{
	Close();

	NV_ENC_STRUCT(NV_ENC_OPEN_ENCODE_SESSION_EX_PARAMS, SessionParams);
	SessionParams.apiVersion = NVENCAPI_VERSION;

	SessionParams.deviceType = SessionDeviceType;
	SessionParams.device = SessionDevice;

	NVENCSTATUS const Result = FAPI::Get<FNVENC>().nvEncOpenEncodeSessionEx(&const_cast<NV_ENC_OPEN_ENCODE_SESSION_EX_PARAMS&>(SessionParams), &Encoder);
	if (Result != NV_ENC_SUCCESS)
	{
		Close();

		return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to re-create encoder"), TEXT("NVENC-CUDA"), Result);
	}

	return EAVResult::Success;
}

FAVResult FEncoderNVENC::ApplyConfig(FVideoEncoderConfigNVENC const& AppliedConfig, FVideoEncoderConfigNVENC& PendingConfig)
{
	if (IsOpen())
	{
		// FVideoEncoderConfigNVENC const& PendingConfig = GetPendingConfig();
		if (AppliedConfig != PendingConfig)
		{
			// Because TransformConfig does not have access to MaxDeviceEncodeWidth/Height, we check this here and re-adjust the PendingConfig maxEncodeWidth/Height if it goes over.
			SetMaxResolution(PendingConfig);

			if (PendingConfig.encodeWidth > MaxDeviceEncodeWidth
				|| PendingConfig.encodeHeight > MaxDeviceEncodeHeight)
			{
				return FAVResult(EAVResult::Error, TEXT("Encoder pending resolution exceeds device max capabilities"));
			}
			if (PendingConfig.encodeWidth > PendingConfig.maxEncodeWidth
				|| PendingConfig.encodeHeight > PendingConfig.maxEncodeHeight)
			{
				return FAVResult(EAVResult::Error, TEXT("Encoder pending resolution exceeds configuration max resolution"));
			}

			if (IsInitialized())
			{
				// Can be reconfigured? See https://docs.nvidia.com/video-technologies/video-codec-sdk/nvenc-video-encoder-api-prog-guide/#reconfigure-api
				if (AppliedConfig.enablePTD == PendingConfig.enablePTD
					&& AppliedConfig.enableEncodeAsync == PendingConfig.enableEncodeAsync
					&& AppliedConfig.encodeConfig->gopLength == PendingConfig.encodeConfig->gopLength
					&& AppliedConfig.encodeConfig->frameIntervalP == PendingConfig.encodeConfig->frameIntervalP
					&& AppliedConfig.encodeConfig->encodeCodecConfig.h264Config.idrPeriod == PendingConfig.encodeConfig->encodeCodecConfig.h264Config.idrPeriod
					&& AppliedConfig.maxEncodeWidth == PendingConfig.maxEncodeWidth
					&& AppliedConfig.maxEncodeHeight == PendingConfig.maxEncodeHeight
					&& PendingConfig.encodeWidth <= PendingConfig.maxEncodeWidth
					&& PendingConfig.encodeHeight <= PendingConfig.maxEncodeHeight)
				{
					NV_ENC_STRUCT(NV_ENC_RECONFIGURE_PARAMS, ReconfigureParams);
					FMemory::Memcpy(&ReconfigureParams.reInitEncodeParams, &static_cast<NV_ENC_INITIALIZE_PARAMS const&>(PendingConfig), sizeof(NV_ENC_INITIALIZE_PARAMS));
					ReconfigureParams.forceIDR = AppliedConfig.encodeWidth != PendingConfig.encodeWidth || AppliedConfig.encodeHeight != PendingConfig.encodeHeight;

					NVENCSTATUS const Result = FAPI::Get<FNVENC>().nvEncReconfigureEncoder(Encoder, &ReconfigureParams);
					if (Result != NV_ENC_SUCCESS)
					{
						return FAVResult(EAVResult::Error, TEXT("Failed to update encoder configuration"), TEXT("NVENC"), Result);
					}
				}
				else
				{
					FAVResult const Result = ReOpen();
					if (Result.IsNotSuccess())
					{
						return Result;
					}
				}
			}

			if (!IsInitialized())
			{
				{
					NVENCSTATUS Result = FAPI::Get<FNVENC>().nvEncInitializeEncoder(Encoder, &const_cast<FVideoEncoderConfigNVENC&>(PendingConfig));
					if (Result != NV_ENC_SUCCESS)
					{
						return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to initialize encoder"), TEXT("NVENC"), Result);
					}
				}

				{
					FAVResult const Result = OnPostInitialization(PendingConfig);
					if (Result.IsNotSuccess())
					{
						return Result;
					}
				}
			}
		}

		return EAVResult::Success;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("NVENC"));
}

int FEncoderNVENC::GetCapability(GUID EncodeGUID, NV_ENC_CAPS CapsToQuery) const
{
	if (IsOpen())
	{
		int CapsValue = 0;

		NV_ENC_STRUCT(NV_ENC_CAPS_PARAM, CapsParam);
		CapsParam.capsToQuery = CapsToQuery;

		NVENCSTATUS const Result = FAPI::Get<FNVENC>().nvEncGetEncodeCaps(Encoder, EncodeGUID, &CapsParam, &CapsValue);
		if (Result != NV_ENC_SUCCESS)
		{
			FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("Failed to query for capability %d"), CapsToQuery), TEXT("NVENC"), Result);

			return 0;
		}

		return CapsValue;
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("NVENC"));
}

void FEncoderNVENC::GetMaxDeviceEncodeResolution(const FVideoEncoderConfigNVENC& PendingConfig)
{
	if (IsOpen())
	{
		if (!bHasMaxDeviceResolution)
		{
			MaxDeviceEncodeWidth = static_cast<uint32>(GetCapability(PendingConfig.encodeGUID, NV_ENC_CAPS_WIDTH_MAX));
			MaxDeviceEncodeHeight = static_cast<uint32>(GetCapability(PendingConfig.encodeGUID, NV_ENC_CAPS_HEIGHT_MAX));
			bHasMaxDeviceResolution = true;
		}
	}
}

void FEncoderNVENC::SetMaxResolution(FVideoEncoderConfigNVENC& PendingConfig)
{
	if (IsOpen())
	{
		GetMaxDeviceEncodeResolution(PendingConfig);

		PendingConfig.maxEncodeWidth = bHasMaxDeviceResolution ? MaxDeviceEncodeWidth : PendingConfig.maxEncodeWidth;
		PendingConfig.maxEncodeHeight = bHasMaxDeviceResolution ? MaxDeviceEncodeHeight : PendingConfig.maxEncodeHeight;
	}
}

FAVResult FEncoderNVENC::SendFrame(NV_ENC_PIC_PARAMS& Input, NV_ENC_LOCK_BITSTREAM& BitstreamLock)
{
	if (IsOpen())
	{
		if (Input.encodePicFlags & NV_ENC_PIC_FLAG_EOS)
		{
			NVENCSTATUS Result = FAPI::Get<FNVENC>().nvEncEncodePicture(Encoder, &Input);
			if (Result != NV_ENC_SUCCESS)
			{
				return FAVResult(EAVResult::Error, TEXT("Error encoding end-of-stream picture"), TEXT("NVENC"), Result);
			}

			return EAVResult::Success;
		}
		else
		{

			NVENCSTATUS Result = FAPI::Get<FNVENC>().nvEncEncodePicture(Encoder, &Input);
			if (Result != NV_ENC_SUCCESS)
			{
				return FAVResult(EAVResult::Error, TEXT("Error encoding picture"), TEXT("NVENC"), Result);
			}

			Result = FAPI::Get<FNVENC>().nvEncLockBitstream(Encoder, &BitstreamLock);
			if (Result != NV_ENC_SUCCESS)
			{
				return FAVResult(EAVResult::ErrorLocking, TEXT("Failed to lock output bitstream"), TEXT("NVENC"), Result);
			}

			TSharedPtr<uint8> const CopiedData = MakeShareable(new uint8[BitstreamLock.bitstreamSizeInBytes]);
			FMemory::BigBlockMemcpy(CopiedData.Get(), BitstreamLock.bitstreamBufferPtr, BitstreamLock.bitstreamSizeInBytes);

			Packets.Enqueue(
				FVideoPacket(
					CopiedData,
					BitstreamLock.bitstreamSizeInBytes,
					BitstreamLock.outputTimeStamp,
					BitstreamLock.frameIdx,
					BitstreamLock.frameAvgQP,
					(BitstreamLock.pictureType & NV_ENC_PIC_TYPE_IDR) != 0));

			Result = FAPI::Get<FNVENC>().nvEncUnlockBitstream(Encoder, BitstreamLock.outputBitstream);
			if (Result != NV_ENC_SUCCESS)
			{
				return FAVResult(EAVResult::ErrorUnlocking, TEXT("Failed to unlock output buffer"), TEXT("NVENC"));
			}

			return EAVResult::Success;
		}
	}

	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("NVENC"));
}

FAVResult FEncoderNVENC::ReceivePacket(FVideoPacket& OutPacket)
{
	if (IsOpen())
	{
		if (Packets.Dequeue(OutPacket))
		{
			return EAVResult::Success;
		}
		
		return EAVResult::PendingInput;
	}
	
	return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("NVENC"));
}

FAVResult FEncoderNVENC::RegisterResource(NV_ENC_REGISTER_RESOURCE& OutRegisterResource, NV_ENC_INPUT_RESOURCE_TYPE ResourceType, NV_ENC_BUFFER_FORMAT ResourceFormat, void* Resource, uint32 Width, uint32 Height, uint32 Stride, NV_ENC_BUFFER_USAGE BufferUsage)
{
	OutRegisterResource.resourceType = ResourceType;
	OutRegisterResource.resourceToRegister = Resource;
	OutRegisterResource.width = Width;
	OutRegisterResource.height = Height;
	OutRegisterResource.pitch = Stride;
	OutRegisterResource.bufferFormat = ResourceFormat;
	OutRegisterResource.bufferUsage = BufferUsage;

	NVENCSTATUS Result = FAPI::Get<FNVENC>().nvEncRegisterResource(Encoder, &OutRegisterResource);
	if (Result != NV_ENC_SUCCESS)
	{
		return FAVResult(EAVResult::ErrorMapping, TEXT("Failed to register frame resource"), TEXT("NVENC"), Result);
	}

	return EAVResult::Success;
}

FAVResult FEncoderNVENC::UnregisterResource(NV_ENC_REGISTERED_PTR RegisteredResource)
{
	if (RegisteredResource != nullptr)
	{
		NVENCSTATUS Result = FAPI::Get<FNVENC>().nvEncUnregisterResource(Encoder, RegisteredResource);
		if (Result != NV_ENC_SUCCESS)
		{
			return FAVResult(EAVResult::ErrorUnmapping, TEXT("Failed to unregister frame resource"), TEXT("NVENC"), Result);
		}
	}

	return EAVResult::Success;
}

FAVResult FEncoderNVENC::MapResource(NV_ENC_MAP_INPUT_RESOURCE& OutMapResource, NV_ENC_REGISTERED_PTR RegisteredResource)
{
	OutMapResource.registeredResource = RegisteredResource;

	NVENCSTATUS Result = FAPI::Get<FNVENC>().nvEncMapInputResource(Encoder, &OutMapResource);
	if (Result != NV_ENC_SUCCESS)
	{
		return FAVResult(EAVResult::ErrorMapping, TEXT("Failed to map frame resource"), TEXT("NVENC"), Result);
	}

	return EAVResult::Success;
}

FAVResult FEncoderNVENC::UnmapResource(NV_ENC_INPUT_PTR MappedResource)
{
	if (MappedResource != nullptr)
	{
		NVENCSTATUS Result = FAPI::Get<FNVENC>().nvEncUnmapInputResource(Encoder, MappedResource);
		if (Result != NV_ENC_SUCCESS)
		{
			return FAVResult(EAVResult::ErrorUnmapping, TEXT("Failed to unmap frame resource"), TEXT("NVENC"), Result);
		}
	}

	return EAVResult::Success;
}

#if PLATFORM_WINDOWS
/**
 * @returns The FString version of HRESULT error message so we can log it.
 **/
const FString AVUtils::AVGetComErrorDescription(HRESULT Res)
{
	const uint32 BufSize = 4096;
	WIDECHAR	 Buffer[4096];
	if (::FormatMessageW(FORMAT_MESSAGE_FROM_SYSTEM,
			nullptr,
			Res,
			MAKELANGID(LANG_NEUTRAL, SUBLANG_NEUTRAL),
			Buffer,
			sizeof(Buffer) / sizeof(*Buffer),
			nullptr))
	{
		return Buffer;
	}
	else
	{
		return TEXT("[cannot find error description]");
	}
}
#endif