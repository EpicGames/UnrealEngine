// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/VideoEncoderNVENC.h"
#include "Templates/AlignmentTemplates.h"

FVideoEncoderNVENCCUDA::~FVideoEncoderNVENCCUDA()
{
	Close();
}

FAVResult FVideoEncoderNVENCCUDA::Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
{
	Close();
	
	TVideoEncoder::Open(NewDevice, NewInstance);

	NV_ENC_STRUCT(NV_ENC_OPEN_ENCODE_SESSION_EX_PARAMS, SessionParams);
	SessionParams.apiVersion = NVENCAPI_VERSION;
	SessionParams.deviceType = NV_ENC_DEVICE_TYPE_CUDA;
	SessionParams.device = GetDevice()->GetContext<FVideoContextCUDA>()->Raw;

	return FEncoderNVENC::Open(NewDevice, NewInstance, SessionParams);
}

void FVideoEncoderNVENCCUDA::Close()
{
	if (IsOpen())
	{
		if (Buffer != nullptr)
		{
			NVENCSTATUS const Result = FAPI::Get<FNVENC>().nvEncDestroyBitstreamBuffer(Encoder, Buffer);
			if (Result != NV_ENC_SUCCESS)
			{
				FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to destroy output buffer"), TEXT("NVENC-CUDA"), Result);
			}

			Buffer = nullptr;
		}

		if (Encoder != nullptr)
		{
			NVENCSTATUS const Result = FAPI::Get<FNVENC>().nvEncDestroyEncoder(Encoder);
			if (Result != NV_ENC_SUCCESS)
			{
				FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to destroy encoder"), TEXT("NVENC-CUDA"), Result);
			}

			Encoder = nullptr;
		}
	}
}

bool FVideoEncoderNVENCCUDA::IsOpen() const
{
	return Encoder != nullptr;
}

FAVResult FVideoEncoderNVENCCUDA::ApplyConfig()
{
	FAVResult Result = FEncoderNVENC::ApplyConfig(AppliedConfig, EditPendingConfig());
	if (Result.IsNotSuccess())
	{
		return Result;
	}

	return TVideoEncoder::ApplyConfig();
}

bool FVideoEncoderNVENCCUDA::IsInitialized() const
{
	return IsOpen() && Buffer != nullptr;
}

FAVResult FVideoEncoderNVENCCUDA::SendFrame(TSharedPtr<FVideoResourceCUDA> const& Resource, uint32 Timestamp, bool bForceKeyframe)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("NVENC-CUDA"));
	}

	FAVResult AVResult = ApplyConfig();
	if (AVResult.IsNotSuccess())
	{
		return AVResult;
	}

	{
		NV_ENC_STRUCT(NV_ENC_REGISTER_RESOURCE, InputRegisterResource);
		NV_ENC_STRUCT(NV_ENC_MAP_INPUT_RESOURCE, InputMapResource);
		NV_ENC_STRUCT(NV_ENC_PIC_PARAMS, Picture);

		ON_SCOPE_EXIT
		{
			if (FAVResult UnmapResult = UnmapResource(InputMapResource.mappedResource); UnmapResult.IsNotSuccess())
			{
				UnmapResult.Log();
			}
		
			if (FAVResult UnregisterResult = UnregisterResource(InputRegisterResource.registeredResource); UnregisterResult.IsNotSuccess())
			{
				UnregisterResult.Log();
			}
		};

		NV_ENC_STRUCT(NV_ENC_LOCK_BITSTREAM, BitstreamLock);
		BitstreamLock.outputBitstream = Buffer;

		if (Resource.IsValid())
		{
			TAVResult<NV_ENC_BUFFER_FORMAT> const ConvertedPixelFormat = FVideoEncoderConfigNVENC::ConvertFormat(Resource->GetFormat());
			if (ConvertedPixelFormat.IsNotSuccess())
			{
				return ConvertedPixelFormat;
			}

			DebugDumpFrame(Resource, Timestamp);

			AVResult = RegisterResource(
				InputRegisterResource,
				NV_ENC_INPUT_RESOURCE_TYPE_CUDAARRAY, // ResourceType
				ConvertedPixelFormat,				  // ResourceFormat
				Resource->GetRaw(),					  // Resource
				Resource->GetWidth(),				  // Width
				Resource->GetHeight(),				  // Height
				Resource->GetStride(),				  // Stride
				NV_ENC_INPUT_IMAGE					  // BufferUsage
			);
			if (AVResult.IsNotSuccess())
			{
				return AVResult;
			}

			AVResult = MapResource(InputMapResource, InputRegisterResource.registeredResource);
			if (AVResult.IsNotSuccess())
			{
				return AVResult;
			}

			Picture.inputWidth = InputRegisterResource.width;
			Picture.inputHeight = InputRegisterResource.height;
			Picture.inputPitch = InputRegisterResource.pitch;
			Picture.inputBuffer = InputMapResource.mappedResource;
			Picture.bufferFmt = InputMapResource.mappedBufferFmt;
			Picture.pictureStruct = NV_ENC_PIC_STRUCT_FRAME;
			Picture.inputTimeStamp = Timestamp;
			Picture.outputBitstream = Buffer;

			if (bForceKeyframe)
			{
				Picture.encodePicFlags |= NV_ENC_PIC_FLAG_FORCEIDR;
			}
		}
		else
		{
			Picture.encodePicFlags = NV_ENC_PIC_FLAG_EOS;
		}

		if (Resource.IsValid())
		{
			Resource->Lock();
		}

		AVResult = FEncoderNVENC::SendFrame(Picture, BitstreamLock);

		if (Resource.IsValid())
		{
			Resource->Unlock();
		}

		if (AVResult.IsNotSuccess())
		{
			// Here we just log the error instead of returning so that we ensure we unmap and unregister the input resources
			AVResult.Log();
		}
	}

	return AVResult;
}

FAVResult FVideoEncoderNVENCCUDA::ReceivePacket(FVideoPacket& OutPacket)
{
	return FEncoderNVENC::ReceivePacket(OutPacket);
}

FAVResult FVideoEncoderNVENCCUDA::OnPostInitialization(FVideoEncoderConfigNVENC& PendingConfig)
{
	NV_ENC_STRUCT(NV_ENC_CREATE_BITSTREAM_BUFFER, CreateBuffer);

	NVENCSTATUS const Result = FAPI::Get<FNVENC>().nvEncCreateBitstreamBuffer(Encoder, &CreateBuffer);
	if (Result != NV_ENC_SUCCESS)
	{
		return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to create output buffer"), TEXT("NVENC-CUDA"), Result);
	}

	Buffer = CreateBuffer.bitstreamBuffer;

	return FAVResult(EAVResult::Success, TEXT("Created output bitstream buffer."), TEXT("NVENC-CUDA"));
}

void FVideoEncoderNVENCCUDA::DebugDumpFrame(TSharedPtr<FVideoResourceCUDA> const& Resource, uint32 Timestamp)
{
	// START DEBUG
#if DEBUG_DUMP_TO_DISK
	{
		TArray64<uint8> OutData;

		Resource->ReadData(OutData);

		FString SaveName = FString::Printf(TEXT("%s/DumpInput/image%05d.%s"), *FPaths::ProjectSavedDir(), Timestamp, ConvertedPixelFormat == NV_ENC_BUFFER_FORMAT_NV12 ? TEXT("nv12") : TEXT("p016"));

		FFileHelper::SaveArrayToFile(OutData, *SaveName);
	}
#endif
	// END DEBUG
}