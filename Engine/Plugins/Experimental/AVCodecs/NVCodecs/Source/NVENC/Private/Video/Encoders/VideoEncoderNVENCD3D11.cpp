// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/VideoEncoderNVENC.h"
#include "Templates/AlignmentTemplates.h"

#if PLATFORM_WINDOWS
/**
 * You may be asking yourself, why on Earth would we need to create a seperate D3D11 device when using NVENC?
 * Well, it basically comes down to the D3D11 device created by Unreal Engine is not compatible with NVENC due to sdk version changes (we think?)
 * and Unreal Engine is in no hurry to bump D3D11 version - so we create our own device. This works only because
 * the D3D11 textures we are using are created as shared resources and accessed through shared handles.
 * Without this chicainery the D3D11 device will be ejected and Unreal Engine and NVENC will crash.
 */
FAVResult CreateD3D11Device(TSharedRef<FAVDevice> const& InDevice, TRefCountPtr<ID3D11Device>& OutEncoderDevice, TRefCountPtr<ID3D11DeviceContext>& OutEncoderDeviceContext)
{
	TRefCountPtr<IDXGIDevice>  DXGIDevice;
	TRefCountPtr<IDXGIAdapter> Adapter;

	HRESULT Result = InDevice->GetContext<FVideoContextD3D11>()->Device->QueryInterface(__uuidof(IDXGIDevice), (void**)DXGIDevice.GetInitReference());
	if (Result != S_OK)
	{
		return FAVResult(EAVResult::Fatal, FString::Printf(TEXT("ID3D11Device::QueryInterface() failed 0x%X - %s."), (uint32)Result, *AVUtils::AVGetComErrorDescription(Result)), TEXT("NVENC-D3D11"));
	}
	else if ((Result = DXGIDevice->GetAdapter(Adapter.GetInitReference())) != S_OK)
	{
		return FAVResult(EAVResult::Fatal, FString::Printf(TEXT("DXGIDevice::GetAdapter() failed 0x%X - %s."), (uint32)Result, *AVUtils::AVGetComErrorDescription(Result)), TEXT("NVENC-D3D11"));
	}

	uint32			  DeviceFlags = 0;
	D3D_FEATURE_LEVEL FeatureLevel = D3D_FEATURE_LEVEL_11_0;
	D3D_FEATURE_LEVEL ActualFeatureLevel;

	if ((Result = D3D11CreateDevice(
			 Adapter,
			 D3D_DRIVER_TYPE_UNKNOWN,
			 NULL,
			 DeviceFlags,
			 &FeatureLevel,
			 1,
			 D3D11_SDK_VERSION,
			 OutEncoderDevice.GetInitReference(),
			 &ActualFeatureLevel,
			 OutEncoderDeviceContext.GetInitReference()))
		!= S_OK)
	{
		return FAVResult(EAVResult::Fatal, FString::Printf(TEXT("D3D11CreateDevice() failed 0x%X - %s."), (uint32)Result, *AVUtils::AVGetComErrorDescription(Result)), TEXT("NVENC-D3D11"));
	}

	return FAVResult(EAVResult::Success, TEXT("Created D3D11 device for NVENC."), TEXT("NVENC-D3D11"));
}

FVideoEncoderNVENCD3D11::~FVideoEncoderNVENCD3D11()
{
	Close();
}

FAVResult FVideoEncoderNVENCD3D11::Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
{
	Close();

	FAVResult Result = CreateD3D11Device(NewDevice, EncoderDevice, EncoderDeviceContext);

	// If we failed to create device early out.
	if (Result.IsNotSuccess())
	{
		return Result;
	}

	TVideoEncoder::Open(NewDevice, NewInstance);

	NV_ENC_STRUCT(NV_ENC_OPEN_ENCODE_SESSION_EX_PARAMS, SessionParams);
	SessionParams.apiVersion = NVENCAPI_VERSION;
	SessionParams.deviceType = NV_ENC_DEVICE_TYPE_DIRECTX;
	SessionParams.device = EncoderDevice;
	// SessionParams.device = GetDevice()->GetContext<FVideoContextD3D11>()->Device;

	return FEncoderNVENC::Open(NewDevice, NewInstance, SessionParams);
}

void FVideoEncoderNVENCD3D11::Close()
{
	if (IsOpen())
	{
		if (Buffer != nullptr)
		{
			NVENCSTATUS const Result = FAPI::Get<FNVENC>().nvEncDestroyBitstreamBuffer(Encoder, Buffer);
			if (Result != NV_ENC_SUCCESS)
			{
				FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to destroy output buffer"), TEXT("NVENC-D3D11"), Result);
			}

			Buffer = nullptr;
		}

		if (Encoder != nullptr)
		{
			NVENCSTATUS const Result = FAPI::Get<FNVENC>().nvEncDestroyEncoder(Encoder);
			if (Result != NV_ENC_SUCCESS)
			{
				FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to destroy encoder"), TEXT("NVENC-D3D11"), Result);
			}

			Encoder = nullptr;
		}
	}
}

bool FVideoEncoderNVENCD3D11::IsOpen() const
{
	return Encoder != nullptr;
}

bool FVideoEncoderNVENCD3D11::IsInitialized() const
{
	return IsOpen() && Buffer != nullptr;
}

FAVResult FVideoEncoderNVENCD3D11::ApplyConfig()
{
	FAVResult Result = FEncoderNVENC::ApplyConfig(AppliedConfig, EditPendingConfig());
	if (Result.IsNotSuccess())
	{
		return Result;
	}

	return TVideoEncoder::ApplyConfig();
}

FAVResult FVideoEncoderNVENCD3D11::SendFrame(TSharedPtr<FVideoResourceD3D11> const& Resource, uint32 Timestamp, bool bForceKeyframe)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("NVENC-D3D11"));
	}

	FAVResult AVResult = ApplyConfig();
	if (AVResult.IsNotSuccess())
	{
		return AVResult;
	}

	TRefCountPtr<ID3D11Texture2D>& Tex = const_cast<TRefCountPtr<ID3D11Texture2D>&>(Resource->GetRaw());

	// Very important: We create a new device for NVENC - we must share out D3D11 texture with the new device by "Opening" it to that device.
	HRESULT HResult = EncoderDevice->OpenSharedResource(Resource->GetSharedHandle(), __uuidof(ID3D11Texture2D), (void**)(Tex.GetInitReference()));
	if (HResult != S_OK)
	{
		return FAVResult(EAVResult::Fatal, TEXT("Failed to open shared handle."), TEXT("NVENC-D3D11"), HResult);
	}

	{
		NV_ENC_STRUCT(NV_ENC_REGISTER_RESOURCE, InputRegisterResource);
		NV_ENC_STRUCT(NV_ENC_MAP_INPUT_RESOURCE, InputMapResource);
		NV_ENC_STRUCT(NV_ENC_PIC_PARAMS, Picture);

		ON_SCOPE_EXIT
		{
			if (FAVResult UnmapResult = UnmapResource(InputMapResource.mappedResource); UnmapResult.IsNotSuccess())
			{
				UnmapResult.Log();
			}
		
			if (FAVResult UnregisterResult = UnregisterResource(InputRegisterResource.registeredResource); UnregisterResult.IsNotSuccess())
			{
				UnregisterResult.Log();
			}
		};

		NV_ENC_STRUCT(NV_ENC_LOCK_BITSTREAM, BitstreamLock);
		BitstreamLock.outputBitstream = Buffer;

		if (Resource.IsValid())
		{
			TAVResult<NV_ENC_BUFFER_FORMAT> const ConvertedPixelFormat = FVideoEncoderConfigNVENC::ConvertFormat(Resource->GetFormat());
			if (ConvertedPixelFormat.IsNotSuccess())
			{
				return ConvertedPixelFormat;
			}

			AVResult = RegisterResource(
				InputRegisterResource,
				NV_ENC_INPUT_RESOURCE_TYPE_DIRECTX, // ResourceType
				ConvertedPixelFormat,				// ResourceFormat
				Resource->GetRaw(),					// Resource
				Resource->GetWidth(),				// Width
				Resource->GetHeight(),				// Height
				Resource->GetStride(),				// Stride
				NV_ENC_INPUT_IMAGE					// BufferUsage
			);
			if (AVResult.IsNotSuccess())
			{
				return AVResult;
			}

			AVResult = MapResource(InputMapResource, InputRegisterResource.registeredResource);
    	    if (AVResult.IsNotSuccess())
			{
				return AVResult;
			}

			Picture.inputWidth = InputRegisterResource.width;
			Picture.inputHeight = InputRegisterResource.height;
			Picture.inputPitch = InputRegisterResource.pitch;
			Picture.inputBuffer = InputMapResource.mappedResource;
			Picture.bufferFmt = InputMapResource.mappedBufferFmt;
			Picture.pictureStruct = NV_ENC_PIC_STRUCT_FRAME;
			Picture.inputTimeStamp = Timestamp;
			Picture.outputBitstream = Buffer;

			if (bForceKeyframe)
			{
				Picture.encodePicFlags |= NV_ENC_PIC_FLAG_FORCEIDR;
			}
		}
		else
		{
			Picture.encodePicFlags = NV_ENC_PIC_FLAG_EOS;
		}

		if (Resource.IsValid())
		{
			Resource->Lock();
		}

		AVResult = FEncoderNVENC::SendFrame(Picture, BitstreamLock);

		if (Resource.IsValid())
		{
			Resource->Unlock();
		}

		if (AVResult.IsNotSuccess())
		{
			// Here we just log the error instead of returning so that we ensure we unmap and unregister the input resources
			AVResult.Log();
		}
	}

	return AVResult;
}

FAVResult FVideoEncoderNVENCD3D11::ReceivePacket(FVideoPacket& OutPacket)
{
	return FEncoderNVENC::ReceivePacket(OutPacket);
}

FAVResult FVideoEncoderNVENCD3D11::OnPostInitialization(FVideoEncoderConfigNVENC& PendingConfig)
{
	NV_ENC_STRUCT(NV_ENC_CREATE_BITSTREAM_BUFFER, CreateBuffer);

	NVENCSTATUS const Result = FAPI::Get<FNVENC>().nvEncCreateBitstreamBuffer(Encoder, &CreateBuffer);
	if (Result != NV_ENC_SUCCESS)
	{
		return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to create output buffer"), TEXT("NVENC-D3D11"), Result);
	}

	Buffer = CreateBuffer.bitstreamBuffer;

	return FAVResult(EAVResult::Success, TEXT("Created output bitstream buffer."), TEXT("NVENC-D3D11"));
}

#endif