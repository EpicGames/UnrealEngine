// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/VideoEncoderNVENC.h"
#include "Templates/AlignmentTemplates.h"

#if PLATFORM_WINDOWS
FAVResult CreateD3D12Fence(TSharedRef<FAVDevice> const& InDevice, TRefCountPtr<ID3D12Fence>& OutFence)
{
	TRefCountPtr<ID3D12Device> D3D12Device = InDevice->GetContext<FVideoContextD3D12>()->Device;

	HRESULT Result;
	if ((Result = D3D12Device->CreateFence(0, D3D12_FENCE_FLAG_NONE, IID_PPV_ARGS(OutFence.GetInitReference()))) != S_OK)
	{
		return FAVResult(EAVResult::Fatal, FString::Printf(TEXT("CreateD3D12Fence() failed 0x%X - %s."), (uint32)Result, *AVUtils::AVGetComErrorDescription(Result)), TEXT("NVENC-D3D12"));
	}

	return FAVResult(EAVResult::Success, TEXT("Created additional D3D12 resources for NVENC."), TEXT("NVENC-D3D12"));
}

FVideoEncoderNVENCD3D12::~FVideoEncoderNVENCD3D12()
{
	Close();
}

FAVResult FVideoEncoderNVENCD3D12::Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
{
	Close();

	FAVResult Result = CreateD3D12Fence(NewDevice, InputFence);
	if (Result.IsNotSuccess())
	{
		return Result;
	}

	Result = CreateD3D12Fence(NewDevice, OutputFence);
	if (Result.IsNotSuccess())
	{
		return Result;
	}

	TVideoEncoder::Open(NewDevice, NewInstance);

	NV_ENC_STRUCT(NV_ENC_OPEN_ENCODE_SESSION_EX_PARAMS, SessionParams);
	SessionParams.apiVersion = NVENCAPI_VERSION;
	SessionParams.deviceType = NV_ENC_DEVICE_TYPE_DIRECTX;
	SessionParams.device = NewDevice->GetContext<FVideoContextD3D12>()->Device;

	return FEncoderNVENC::Open(NewDevice, NewInstance, SessionParams);
}

void FVideoEncoderNVENCD3D12::Close()
{
	if (IsOpen())
	{
		if (Encoder != nullptr)
		{
			NVENCSTATUS const Result = FAPI::Get<FNVENC>().nvEncDestroyEncoder(Encoder);
			if (Result != NV_ENC_SUCCESS)
			{
				FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to destroy encoder"), TEXT("NVENC-D3D11"), Result);
			}

			Encoder = nullptr;
		}
	}
}

bool FVideoEncoderNVENCD3D12::IsOpen() const
{
	return Encoder != nullptr;
}

bool FVideoEncoderNVENCD3D12::IsInitialized() const
{
	return IsOpen() && OutputBitstreamResource.IsValid();
}

FAVResult FVideoEncoderNVENCD3D12::ApplyConfig()
{
	FAVResult Result = FEncoderNVENC::ApplyConfig(AppliedConfig, EditPendingConfig());
	if (Result.IsNotSuccess())
	{
		return Result;
	}

	return TVideoEncoder::ApplyConfig();
}

FAVResult FVideoEncoderNVENCD3D12::SendFrame(TSharedPtr<FVideoResourceD3D12> const& Resource, uint32 Timestamp, bool bForceKeyframe)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("NVENC-D3D12"));
	}

	{
		FVideoEncoderConfigNVENC& PendingConfig = EditPendingConfig();

		TAVResult<NV_ENC_BUFFER_FORMAT> const ConvertedPixelFormat = FVideoEncoderConfigNVENC::ConvertFormat(Resource->GetFormat());
		if (ConvertedPixelFormat.IsNotSuccess())
		{
			return ConvertedPixelFormat;
		}

		if (ConvertedPixelFormat != NV_ENC_BUFFER_FORMAT_ARGB)
		{
			return FAVResult(EAVResult::ErrorUnsupported, TEXT("DX12 resources must be of the format DXGI_FORMAT_B8G8R8A8_UNORM"), TEXT("NVENC"));
		}

		PendingConfig.bufferFormat = ConvertedPixelFormat;

		FAVResult AVResult = ApplyConfig();
		if (AVResult.IsNotSuccess())
		{
			return AVResult;
		}
	}

	FAVResult	AVResult = EAVResult::Success;

	{
		NV_ENC_STRUCT(NV_ENC_REGISTER_RESOURCE, InputRegisterResource);
		NV_ENC_STRUCT(NV_ENC_MAP_INPUT_RESOURCE, InputMapResource);
		NV_ENC_STRUCT(NV_ENC_REGISTER_RESOURCE, OutputRegisterResource);
		NV_ENC_STRUCT(NV_ENC_MAP_INPUT_RESOURCE, OutputMapResource);

		ON_SCOPE_EXIT
		{
			// Unmap and unregister input resources
			if (FAVResult UnmapResult = UnmapResource(InputMapResource.mappedResource); UnmapResult.IsNotSuccess())
			{
				UnmapResult.Log();
			}

			if (FAVResult UnregisterResult = UnregisterResource(InputRegisterResource.registeredResource); UnregisterResult.IsNotSuccess())
			{
				UnregisterResult.Log();
			}

			// Unmap and unregister output resources
			if (FAVResult UnmapResult = UnmapResource(OutputMapResource.mappedResource); UnmapResult.IsNotSuccess())
			{
				UnmapResult.Log();
			}

			if (FAVResult UnregisterResult = UnregisterResource(OutputRegisterResource.registeredResource); UnregisterResult.IsNotSuccess())
			{
				UnregisterResult.Log();
			}
		};

		// Register and map input resource
		NV_ENC_STRUCT(NV_ENC_INPUT_RESOURCE_D3D12, InputResource);
		if (Resource.IsValid())
		{
			AVResult = RegisterResource(
				InputRegisterResource,
				NV_ENC_INPUT_RESOURCE_TYPE_DIRECTX, // ResourceType
				NV_ENC_BUFFER_FORMAT_ARGB,			// ResourceFormat
				Resource->GetResource(),			// Resource
				Resource->GetWidth(),				// Width
				Resource->GetHeight(),				// Height
				0,									// Stride
				NV_ENC_INPUT_IMAGE					// BufferUsage
			);
			if (AVResult.IsNotSuccess())
			{
				return AVResult;
			}

			AVResult = MapResource(InputMapResource, InputRegisterResource.registeredResource);
			if (AVResult.IsNotSuccess())
			{
				return AVResult;
			}

			InputResource.inputFencePoint.pFence = InputFence.GetReference();
			InputResource.pInputBuffer = InputMapResource.mappedResource;
			InputResource.inputFencePoint.waitValue = InputFenceVal;
			InputResource.inputFencePoint.bWait = true;
		}

		FPlatformAtomics::InterlockedIncrement(&OutputFenceVal);

		// Register and map output resource
		NV_ENC_STRUCT(NV_ENC_OUTPUT_RESOURCE_D3D12, OutputResource);
		{
			AVResult = RegisterResource(
				OutputRegisterResource,
				NV_ENC_INPUT_RESOURCE_TYPE_DIRECTX,										  // ResourceType
				NV_ENC_BUFFER_FORMAT_U8,												  // ResourceFormat
				OutputBitstreamResource.GetReference(),									  // Resource
				Align(AppliedConfig.encodeWidth * AppliedConfig.encodeHeight * 4 * 2, 4), // Width
				1,																		  // Height
				0,																		  // Stride
				NV_ENC_OUTPUT_BITSTREAM													  // BufferUsage
			);
			if (AVResult.IsNotSuccess())
			{
				return AVResult;
			}

			AVResult = MapResource(OutputMapResource, OutputRegisterResource.registeredResource);
			if (AVResult.IsNotSuccess())
			{
				return AVResult;
			}

			OutputResource.outputFencePoint.pFence = OutputFence.GetReference();
			OutputResource.pOutputBuffer = OutputMapResource.mappedResource;
			OutputResource.outputFencePoint.signalValue = OutputFenceVal;
			OutputResource.outputFencePoint.bSignal = true;
		}

		NV_ENC_STRUCT(NV_ENC_PIC_PARAMS, Picture);
		if (Resource.IsValid())
		{
			Picture.pictureStruct = NV_ENC_PIC_STRUCT_FRAME;
			Picture.inputTimeStamp = Timestamp;
			Picture.inputBuffer = &InputResource;
			Picture.bufferFmt = InputMapResource.mappedBufferFmt;
			Picture.inputWidth = InputRegisterResource.width;
			Picture.inputHeight = InputRegisterResource.height;
			Picture.outputBitstream = &OutputResource;
			Picture.completionEvent = nullptr;

			if (bForceKeyframe)
			{
				Picture.encodePicFlags |= NV_ENC_PIC_FLAG_FORCEIDR;
			}
		}
		else
		{
			Picture.encodePicFlags = NV_ENC_PIC_FLAG_EOS;
		}

		if (Resource.IsValid())
		{
			Resource->Lock();
		}

		NV_ENC_STRUCT(NV_ENC_LOCK_BITSTREAM, BitstreamLock);
		BitstreamLock.outputBitstream = &OutputResource;

		AVResult = FEncoderNVENC::SendFrame(Picture, BitstreamLock);

		if (Resource.IsValid())
		{
			Resource->Unlock();
		}

		if (AVResult.IsNotSuccess())
		{
			// Here we just log the error instead of returning so that we ensure we unmap and unregister the input resources
			AVResult.Log();
		}
	}

	return AVResult;
}

FAVResult FVideoEncoderNVENCD3D12::ReceivePacket(FVideoPacket& OutPacket)
{
	return FEncoderNVENC::ReceivePacket(OutPacket);
}

FAVResult FVideoEncoderNVENCD3D12::OnPostInitialization(FVideoEncoderConfigNVENC& PendingConfig)
{
	TRefCountPtr<ID3D12Device> D3D12Device = GetDevice()->GetContext<FVideoContextD3D12>()->Device;

	D3D12_HEAP_PROPERTIES HeapProps = {};
	HeapProps.Type = D3D12_HEAP_TYPE_READBACK;
	HeapProps.CPUPageProperty = D3D12_CPU_PAGE_PROPERTY_UNKNOWN;
	HeapProps.MemoryPoolPreference = D3D12_MEMORY_POOL_UNKNOWN;

	D3D12_RESOURCE_DESC ResourceDesc = {};
	ResourceDesc.Dimension = D3D12_RESOURCE_DIMENSION_BUFFER;
	ResourceDesc.Alignment = 0;
	ResourceDesc.Width = Align(PendingConfig.encodeWidth * PendingConfig.encodeHeight * 4 * 2, 4);
	ResourceDesc.Height = 1;
	ResourceDesc.DepthOrArraySize = 1;
	ResourceDesc.MipLevels = 1;
	ResourceDesc.Format = DXGI_FORMAT_UNKNOWN;
	ResourceDesc.SampleDesc.Count = 1;
	ResourceDesc.SampleDesc.Quality = 0;
	ResourceDesc.Layout = D3D12_TEXTURE_LAYOUT_ROW_MAJOR;
	ResourceDesc.Flags = D3D12_RESOURCE_FLAG_NONE;

	HRESULT Result;
	if ((Result = D3D12Device->CreateCommittedResource(&HeapProps, D3D12_HEAP_FLAG_NONE, &ResourceDesc, D3D12_RESOURCE_STATE_COPY_DEST, nullptr, IID_PPV_ARGS(OutputBitstreamResource.GetInitReference()))) != S_OK)
	{
		return FAVResult(EAVResult::Fatal, FString::Printf(TEXT("CreateCommittedResource() failed 0x%X - %s."), (uint32)Result, *AVUtils::AVGetComErrorDescription(Result)), TEXT("NVENC-D3D12"));
	}

	return FAVResult(EAVResult::Success, TEXT("Created additional D3D12 resources for NVENC."), TEXT("NVENC-D3D12"));
}

#endif