// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVUtility.h"

THIRD_PARTY_INCLUDES_START

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

#include <nvEncodeAPI.h>

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_END

#define NV_ENC_STRUCT(OfType, VarName)	\
	OfType	VarName;					\
	FMemory::Memzero(VarName);			\
	VarName.version = OfType ## _VER

class FNVENC : public FAPI, public NV_ENCODE_API_FUNCTION_LIST
{
public:
	bool bHasCompatibleGPU = true;

	NVENC_API FNVENC();

	NVENC_API virtual bool IsValid() const override;

	NVENC_API FString GetErrorString(void* Encoder, NVENCSTATUS ForStatus) const;

private:
	void* DllHandle = nullptr;
};

DECLARE_TYPEID(FNVENC, NVENC_API);
