// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVExtension.h"
#include "Video/VideoEncoder.h"

#include "NVENC.h"

#define UE_API NVENC_API

struct FVideoEncoderConfigNVENC : public FAVConfig, public NV_ENC_INITIALIZE_PARAMS
{
public:
	static UE_API TAVResult<NV_ENC_PARAMS_RC_MODE> ConvertRateControlMode(ERateControlMode Mode);
	static UE_API TAVResult<ERateControlMode> ConvertRateControlMode(NV_ENC_PARAMS_RC_MODE Mode);
	static UE_API TAVResult<NV_ENC_MULTI_PASS> ConvertMultipassMode(EMultipassMode Mode);
	static UE_API TAVResult<EMultipassMode> ConvertMultipassMode(NV_ENC_MULTI_PASS Mode);
	static UE_API TAVResult<NV_ENC_BUFFER_FORMAT> ConvertFormat(EVideoFormat const& Format);
	
private:
	NV_ENC_CONFIG Config = {};

public:
	FVideoEncoderConfigNVENC()
		: FAVConfig()
		, NV_ENC_INITIALIZE_PARAMS({})
	{
		version = NV_ENC_INITIALIZE_PARAMS_VER;
		presetGUID = NV_ENC_PRESET_P1_GUID;
		frameRateNum = 60;
		frameRateDen = 1;
		enablePTD = 1;
		reportSliceOffsets = 0;
		enableSubFrameWrite = 0;
		maxEncodeWidth = 0;
		maxEncodeHeight = 0;

		Config.version = NV_ENC_CONFIG_VER;
		encodeConfig = &Config;

		// From the NVENC ultra low latency preset
		Config.gopLength = -1;
		Config.frameIntervalP = 1;
		Config.frameFieldMode = NV_ENC_PARAMS_FRAME_FIELD_MODE_FRAME;
		Config.mvPrecision = NV_ENC_MV_PRECISION_QUARTER_PEL;//NV_ENC_MV_PRECISION_DEFAULT
		Config.rcParams.constQP = { 28, 31, 25 };
		Config.rcParams.lowDelayKeyFrameScale = 1;
		Config.rcParams.multiPass = NV_ENC_TWO_PASS_QUARTER_RESOLUTION;//NV_ENC_MULTI_PASS_DISABLED
	}
		
	FVideoEncoderConfigNVENC(FVideoEncoderConfigNVENC const& Other)
	{
		*this = Other;
	}

	FVideoEncoderConfigNVENC& operator=(FVideoEncoderConfigNVENC const& Other)
	{
		FMemory::Memcpy(*this, Other);
		encodeConfig = &Config;

		FMemory::Memcpy(Config, Other.Config);

		return *this;
	}

	bool operator==(FVideoEncoderConfigNVENC const& Other) const
	{
		// We want to compare the encodeConfig member's contents not its memory address
		// so we temporarily store "this" encodeConfig and set "this" encodeConfig to
		// the same object as the one in Other, so it is essentially not considered in
		// a memcmp of "this" and "Other"
		NV_ENC_CONFIG* const TempEncodeConfig = encodeConfig;
		const_cast<FVideoEncoderConfigNVENC*>(this)->encodeConfig = Other.encodeConfig;

		// Compare the data of "this" and Other, but also the data of 
		// "this"->encodeConfig (currently stored in TempEncodeConfig) and
		// Other.encodeConfig, true if both comparison find the data to be identical
		bool const IsEqual = FMemory::Memcmp(this, &Other, sizeof(FVideoEncoderConfigNVENC)) == 0
			&& FMemory::Memcmp(TempEncodeConfig, Other.encodeConfig, sizeof(NV_ENC_CONFIG)) == 0;

		// Set "this" encodeConfig back to its original value
		const_cast<FVideoEncoderConfigNVENC*>(this)->encodeConfig = TempEncodeConfig;

		return IsEqual;
	}

	bool operator!=(FVideoEncoderConfigNVENC const& Other) const
	{
		return !(*this == Other);
	}
};

template <>
FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigNVENC& OutConfig, struct FVideoEncoderConfig const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(struct FVideoEncoderConfig& OutConfig, FVideoEncoderConfigNVENC const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigNVENC& OutConfig, struct FVideoEncoderConfigH264 const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigNVENC& OutConfig, struct FVideoEncoderConfigH265 const& InConfig);

template <>
FAVResult FAVExtension::TransformConfig(FVideoEncoderConfigNVENC& OutConfig, struct FVideoEncoderConfigAV1 const& InConfig);

DECLARE_TYPEID(FVideoEncoderConfigNVENC, NVENC_API);

#undef UE_API
