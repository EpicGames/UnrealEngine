// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Queue.h"
#include "Misc/ScopeExit.h"
#include "NVENC.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/RefCounting.h"
#include "Video/Encoders/Configs/VideoEncoderConfigNVENC.h"
#include "Video/Resources/VideoResourceCUDA.h"
#include "Video/VideoEncoder.h"
#include "Video/VideoResource.h"

#if PLATFORM_WINDOWS
	#include "Video/Resources/D3D/VideoResourceD3D.h"
#endif // PLATFORM_WINDOWS

#if PLATFORM_WINDOWS
namespace AVUtils
{
	const FString AVGetComErrorDescription(HRESULT Res);
}
#endif

/**
 * This class is the core of the NVENC encoder logic. Specialisations should inherit from this class to reduce duplicated logic
 **/
class FEncoderNVENC
{
private:
	uint32 MaxDeviceEncodeWidth = 0;
	uint32 MaxDeviceEncodeHeight = 0;
	bool bHasMaxDeviceResolution = false;

	NVENC_API void SetMaxResolution(FVideoEncoderConfigNVENC& PendingConfig);
	NVENC_API void GetMaxDeviceEncodeResolution(const FVideoEncoderConfigNVENC& PendingConfig);

public:
	FEncoderNVENC() = default;
	virtual ~FEncoderNVENC() = default;

protected:
	NV_ENC_DEVICE_TYPE  SessionDeviceType;
	void* SessionDevice;

	void* Encoder = nullptr;
	TQueue<FVideoPacket> Packets;

	NVENC_API FAVResult ReOpen();
	NVENC_API FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance, NV_ENC_OPEN_ENCODE_SESSION_EX_PARAMS& EncoderSessionParams);
	NVENC_API FAVResult ApplyConfig(FVideoEncoderConfigNVENC const& AppliedConfig, FVideoEncoderConfigNVENC& PendingConfig);
	NVENC_API FAVResult SendFrame(NV_ENC_PIC_PARAMS& Input, NV_ENC_LOCK_BITSTREAM& BitstreamLock);
	NVENC_API FAVResult ReceivePacket(FVideoPacket& OutPacket);

	//
	virtual bool IsOpen() const = 0;
	virtual void Close() = 0;
	virtual	bool IsInitialized() const = 0;
	virtual FAVResult OnPostInitialization(FVideoEncoderConfigNVENC& PendingConfig) { return EAVResult::Success; }
	//

	NVENC_API FAVResult RegisterResource(NV_ENC_REGISTER_RESOURCE& OutRegisterResource, NV_ENC_INPUT_RESOURCE_TYPE ResourceType, NV_ENC_BUFFER_FORMAT ResourceFormat, void* Resource, uint32 Width, uint32 Height, uint32 Stride, NV_ENC_BUFFER_USAGE BufferUsage);
	NVENC_API FAVResult UnregisterResource(NV_ENC_REGISTERED_PTR RegisteredResource);
	NVENC_API FAVResult MapResource(NV_ENC_MAP_INPUT_RESOURCE& OutMapResource, NV_ENC_REGISTERED_PTR RegisteredResource);
	NVENC_API FAVResult UnmapResource(NV_ENC_INPUT_PTR MappedResource);

	NVENC_API int GetCapability(GUID EncodeGUID, NV_ENC_CAPS CapsToQuery) const;
};

/**
 *  An AVCodecs VideoEncoder that takes a FVideoResourceCUDA.
 **/
class FVideoEncoderNVENCCUDA : public TVideoEncoder<FVideoResourceCUDA, FVideoEncoderConfigNVENC>, public FEncoderNVENC
{
private:
	NV_ENC_OUTPUT_PTR Buffer = nullptr;

public:
	FVideoEncoderNVENCCUDA() = default;
	NVENC_API virtual ~FVideoEncoderNVENCCUDA() override;

	// Begin TVideoEncoder interface
	NVENC_API virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override;
	NVENC_API virtual void Close() override;
	NVENC_API virtual bool IsOpen() const override;
	NVENC_API virtual FAVResult ApplyConfig() override;
	NVENC_API virtual FAVResult SendFrame(TSharedPtr<FVideoResourceCUDA> const& Resource, uint32 Timestamp, bool bForceKeyframe = false) override;
	NVENC_API virtual FAVResult ReceivePacket(FVideoPacket& OutPacket) override;
	// End TVideoEncoder interface

	// Begin FEncoderNVENC interface
	NVENC_API virtual	bool IsInitialized() const override;
	NVENC_API virtual FAVResult OnPostInitialization(FVideoEncoderConfigNVENC& PendingConfig) override;
	// End FEncoderNVENC interface

private:
	NVENC_API void DebugDumpFrame(TSharedPtr<FVideoResourceCUDA> const& Resource, uint32 Timestamp);
};

#if PLATFORM_WINDOWS
/**
 *  An AVCodecs VideoEncoder that takes a FVideoResourceD3D11.
 **/
class FVideoEncoderNVENCD3D11 : public TVideoEncoder<FVideoResourceD3D11, FVideoEncoderConfigNVENC>, public FEncoderNVENC
{
private:
	TRefCountPtr<ID3D11Device> EncoderDevice;
	TRefCountPtr<ID3D11DeviceContext> EncoderDeviceContext;

	NV_ENC_OUTPUT_PTR Buffer = nullptr;

public:
	FVideoEncoderNVENCD3D11() = default;
	NVENC_API virtual ~FVideoEncoderNVENCD3D11() override;

	// Begin TVideoEncoder interface
	NVENC_API virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override;
	NVENC_API virtual void Close() override;
	NVENC_API virtual bool IsOpen() const override;
	NVENC_API virtual FAVResult ApplyConfig() override;
	NVENC_API virtual FAVResult SendFrame(TSharedPtr<FVideoResourceD3D11> const& Resource, uint32 Timestamp, bool bForceKeyframe = false) override;
	NVENC_API virtual FAVResult ReceivePacket(FVideoPacket& OutPacket) override;
	// End TVideoEncoder interface

	// Begin FEncoderNVENC interface
	NVENC_API virtual	bool IsInitialized() const override;
	NVENC_API virtual FAVResult OnPostInitialization(FVideoEncoderConfigNVENC& PendingConfig) override;
	// End FEncoderNVENC interface
};

/**
 *  An AVCodecs VideoEncoder that takes a FVideoResourceD3D12.
 **/
class FVideoEncoderNVENCD3D12 : public TVideoEncoder<FVideoResourceD3D12, FVideoEncoderConfigNVENC>, public FEncoderNVENC
{
private:
	TRefCountPtr<ID3D12Fence> InputFence;
	int64 InputFenceVal = 0;

	TRefCountPtr<ID3D12Fence> OutputFence;
	int64 OutputFenceVal = 0;

	TRefCountPtr<ID3D12Resource> OutputBitstreamResource;

public:
	FVideoEncoderNVENCD3D12() = default;
	NVENC_API virtual ~FVideoEncoderNVENCD3D12() override;

	// Begin TVideoEncoder interface
	NVENC_API virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override;
	NVENC_API virtual void Close() override;
	NVENC_API virtual bool IsOpen() const override;
	NVENC_API virtual FAVResult ApplyConfig() override;
	NVENC_API virtual FAVResult SendFrame(TSharedPtr<FVideoResourceD3D12> const& Resource, uint32 Timestamp, bool bForceKeyframe = false) override;
	NVENC_API virtual FAVResult ReceivePacket(FVideoPacket& OutPacket) override;
	// End TVideoEncoder interface

	// Begin FEncoderNVENC interface
	NVENC_API virtual	bool IsInitialized() const override;
	NVENC_API virtual FAVResult OnPostInitialization(FVideoEncoderConfigNVENC& PendingConfig) override;
	// End FEncoderNVENC interface
};
#endif // PLATFORM_WINDOWS
